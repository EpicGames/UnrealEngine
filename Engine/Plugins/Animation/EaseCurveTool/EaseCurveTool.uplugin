{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Ease Curve Tool",
	"Description": "Sequencer / Curve Editor tool to ease tangents between keyframes using custom preset libraries",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "EaseCurveTool",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
