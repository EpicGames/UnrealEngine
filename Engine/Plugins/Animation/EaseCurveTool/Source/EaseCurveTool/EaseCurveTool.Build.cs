// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EaseCurveTool : ModuleRules
{
	public EaseCurveTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetDefinition",
			"ApplicationCore",
			"ContentBrowser",
			"CoreUObject",
			"CurveEditor",
			"DeveloperSettings",
			"EditorSubsystem",
			"Engine",
			"InputCore",
			"Json",
			"MovieScene",
			"MovieSceneTools",
			"Projects",
			"PropertyEditor",
			"Sequencer",
			"SequencerCore",
			"Slate",
			"SlateCore",
			"ToolMenus",
			"UnrealEd",
			"WorkspaceMenuStructure"
		});
	}
}
