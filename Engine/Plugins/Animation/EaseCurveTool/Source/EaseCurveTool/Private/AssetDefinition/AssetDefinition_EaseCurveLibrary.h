// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "EaseCurveLibrary.h"
#include "AssetDefinition_EaseCurveLibrary.generated.h"

UCLASS()
class UAssetDefinition_EaseCurveLibrary : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_EaseCurveLibrary", "Ease Curve Library"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(201, 29, 85)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UEaseCurveLibrary::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_EaseCurveLibrarySubMenu", "Curve"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
