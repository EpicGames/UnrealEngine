// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurve.h"

UEaseCurve::UEaseCurve()
{
	StartKeyHandle = FloatCurve.AddKey(0, 0);
	EndKeyHandle = FloatCurve.AddKey(1, 1);
}

void UEaseCurve::OnCurveChanged(const TArray<FRichCurveEditInfo>& InRichCurveEditInfos) 
{
	UCurveFloat::OnCurveChanged(InRichCurveEditInfos);
}

FKeyHandle UEaseCurve::GetStartKeyHandle() const
{
	return StartKeyHandle;
}

FKeyHandle UEaseCurve::GetEndKeyHandle() const
{
	return EndKeyHandle;
}

FRichCurveKey& UEaseCurve::GetStartKey()
{
	return FloatCurve.GetKey(StartKeyHandle);
}

FRichCurveKey& UEaseCurve::GetEndKey()
{
	return FloatCurve.GetKey(EndKeyHandle);
}

float UEaseCurve::GetStartKeyTime() const
{
	return FloatCurve.GetKey(StartKeyHandle).Time;
}

float UEaseCurve::GetStartKeyValue() const
{
	return FloatCurve.GetKey(StartKeyHandle).Value;
}

float UEaseCurve::GetEndKeyTime() const
{
	return FloatCurve.GetKey(EndKeyHandle).Time;
}

float UEaseCurve::GetEndKeyValue() const
{
	return FloatCurve.GetKey(EndKeyHandle).Value;
}

FEaseCurveTangents UEaseCurve::GetTangents() const
{
	return FEaseCurveTangents(FloatCurve.GetKey(StartKeyHandle), FloatCurve.GetKey(EndKeyHandle));
}

void UEaseCurve::SetTangents(const FEaseCurveTangents& InTangents)
{
	SetStartTangent(InTangents.Start, InTangents.StartWeight);
	SetEndTangent(InTangents.End, InTangents.EndWeight);
}

void UEaseCurve::SetStartTangent(const float InTangent, const float InTangentWeight)
{
	FloatCurve.SetKeyTime(StartKeyHandle, 0);
	FloatCurve.SetKeyValue(StartKeyHandle, 0);
	FloatCurve.SetKeyInterpMode(StartKeyHandle, ERichCurveInterpMode::RCIM_Cubic);
	FloatCurve.SetKeyTangentWeightMode(StartKeyHandle, ERichCurveTangentWeightMode::RCTWM_WeightedBoth, false);
	FloatCurve.SetKeyTangentMode(StartKeyHandle, ERichCurveTangentMode::RCTM_Break, false);

	FRichCurveKey& StartKey = GetStartKey();
	StartKey.ArriveTangent = 0;
	StartKey.ArriveTangentWeight = 0;
	StartKey.LeaveTangent = InTangent;
	StartKey.LeaveTangentWeight = InTangentWeight;
}

void UEaseCurve::SetEndTangent(const float InTangent, const float InTangentWeight)
{
	FloatCurve.SetKeyTime(EndKeyHandle, 1);
	FloatCurve.SetKeyValue(EndKeyHandle, 1);
	FloatCurve.SetKeyInterpMode(EndKeyHandle, ERichCurveInterpMode::RCIM_Cubic);
	FloatCurve.SetKeyTangentWeightMode(EndKeyHandle, ERichCurveTangentWeightMode::RCTWM_WeightedBoth, false);
	FloatCurve.SetKeyTangentMode(EndKeyHandle, ERichCurveTangentMode::RCTM_Break, false);

	FRichCurveKey& EndKey = GetEndKey();
	EndKey.LeaveTangent = 0;
	EndKey.LeaveTangentWeight = 0;
	EndKey.ArriveTangent = InTangent;
	EndKey.ArriveTangentWeight = InTangentWeight;
}

void UEaseCurve::FlattenOrStraightenTangents(const FKeyHandle InKeyHandle, const bool bInFlattenTangents)
{
	FRichCurveKey& Key = FloatCurve.GetKey(InKeyHandle);

	float LeaveTangent = Key.LeaveTangent;
	float ArriveTangent = Key.ArriveTangent;

	if (bInFlattenTangents)
	{
		LeaveTangent = 0.f;
		ArriveTangent = 0.f;
	}
	else
	{
		LeaveTangent = (LeaveTangent + ArriveTangent) * 0.5f;
		ArriveTangent = LeaveTangent;
	}

	Key.LeaveTangent = LeaveTangent;
	Key.ArriveTangent = ArriveTangent;
	if (Key.InterpMode == RCIM_Cubic && Key.TangentMode == RCTM_Auto)
	{
		Key.TangentMode = RCTM_User;
	}
}

void UEaseCurve::SetInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode)
{
	if (InInterpMode != RCIM_Cubic)
	{
		FloatCurve.SetKeyInterpMode(StartKeyHandle, InInterpMode);
		FloatCurve.SetKeyTangentMode(StartKeyHandle, InTangentMode);

		FloatCurve.SetKeyInterpMode(EndKeyHandle, InInterpMode);
		FloatCurve.SetKeyTangentMode(EndKeyHandle, InTangentMode);
	}
}

void UEaseCurve::BroadcastUpdate()
{
	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	ChangedCurveEditInfos.Add(FRichCurveEditInfo(&FloatCurve));
	OnCurveChanged(ChangedCurveEditInfos);
}
