// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTangents.h"
#include "Curves/CurveFloat.h"
#include "EaseCurve.generated.h"

UCLASS()
class UEaseCurve : public UCurveFloat 
{
	GENERATED_BODY()

public:
	UEaseCurve();

	FKeyHandle GetStartKeyHandle() const;
	FKeyHandle GetEndKeyHandle() const;

	FRichCurveKey& GetStartKey();
	FRichCurveKey& GetEndKey();

	float GetStartKeyTime() const;
	float GetStartKeyValue() const;
	float GetEndKeyTime() const;
	float GetEndKeyValue() const;

	FEaseCurveTangents GetTangents() const;
	void SetTangents(const FEaseCurveTangents& InTangents);
	void SetStartTangent(const float InTangent, const float InTangentWeight);
	void SetEndTangent(const float InTangent, const float InTangentWeight);

	void FlattenOrStraightenTangents(const FKeyHandle InKeyHandle, const bool bInFlattenTangents);

	void SetInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode);

	void BroadcastUpdate();

	//~ Begin UCurveFloat
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& InRichCurveEditInfos) override;
	//~ End UCurveFloat

protected:
	FKeyHandle StartKeyHandle;
	FKeyHandle EndKeyHandle;
};
