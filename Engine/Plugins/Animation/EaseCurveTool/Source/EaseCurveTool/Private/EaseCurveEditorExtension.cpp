// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveEditorExtension.h"
#include "EaseCurvePreset.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolExtender.h"
#include "EaseCurveToolSettings.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Menus/CurveEditorToolBarMenu.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "EaseCurveEditorExtension"

namespace UE::EaseCurveTool
{

FEaseCurveEditorExtension::FEaseCurveEditorExtension(const TWeakPtr<FCurveEditor>& InWeakCurveEditor)
	: WeakCurveEditor(InWeakCurveEditor)
{
}

void FEaseCurveEditorExtension::BindCommands(TSharedRef<FUICommandList> InCommandList)
{
	
}

TSharedPtr<FExtender> FEaseCurveEditorExtension::MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList)
{
	const TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	Extender->AddToolBarExtension(TEXT("Tangents"),
		EExtensionHook::After,
		InCommandList,
		FToolBarExtensionDelegate::CreateRaw(this, &FEaseCurveEditorExtension::ExtendCurveEditorToolbar));

	return Extender;
}

void FEaseCurveEditorExtension::ExtendCurveEditorToolbar(FToolBarBuilder& ToolBarBuilder)
{
	ToolBarBuilder.BeginSection(TEXT("EaseCurveTool"));

	ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	const FSlateIcon Icon = FSlateIcon(FEaseCurveStyle::Get().GetStyleSetName(), TEXT("Icon.ToolBar"));

	ToolBarBuilder.AddToolBarButton(FUIAction(
			FExecuteAction::CreateRaw(this, &FEaseCurveEditorExtension::PopupToolbarMenu),
			FCanExecuteAction::CreateRaw(this, &FEaseCurveEditorExtension::IsButtonEnabled))
		, NAME_None
		, LOCTEXT("EaseCurvePresetsLabel", "Ease Curve Presets")
		, TAttribute<FText>::CreateRaw(this, &FEaseCurveEditorExtension::GetEaseCurveTooltipText)
		, Icon);

	ToolBarBuilder.AddComboButton(FUIAction(
			FExecuteAction(),
			FCanExecuteAction::CreateRaw(this, &FEaseCurveEditorExtension::IsButtonEnabled))
		, FOnGetContent::CreateRaw(this, &FEaseCurveEditorExtension::GetToolbarButtonMenuContent)
		, LOCTEXT("QuickEaseCurveLabel", "Quick Ease Curve")
		, TAttribute<FText>::CreateRaw(this, &FEaseCurveEditorExtension::GetEaseCurveTooltipText)
		, Icon
		, true
		, NAME_None
		, TAttribute<EVisibility>::CreateRaw(this, &FEaseCurveEditorExtension::GetToolbarButtonVisibility));

	ToolBarBuilder.EndSection();
}

void FEaseCurveEditorExtension::PopupToolbarMenu()
{
	const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance();
	if (!ToolInstance.IsValid())
	{
		return;
	}

	const Slate::FDeprecateVector2DResult CursorPosition = FSlateApplication::Get().GetCursorPos();
	const FWidgetPath WidgetPath = FSlateApplication::Get().LocateWindowUnderMouse(CursorPosition
		, FSlateApplication::Get().GetInteractiveTopLevelWindows());
	if (!WidgetPath.IsValid())
	{
		return;
	}

	const TSharedPtr<SButton> ButtonWidget = StaticCastSharedRef<SButton>(WidgetPath.GetLastWidget());
	if (!ButtonWidget.IsValid())
	{
		return;
	}

	const FVector2D SummonLocation = FSlateApplication::Get().GetCursorPos();

	ButtonMenu = MakeShared<FCurveEditorToolBarMenu>(GetToolInstance());

	FSlateApplication::Get().PushMenu(FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef()
		, WidgetPath//FWidgetPath()
		, ButtonMenu->GenerateWidget()
		, SummonLocation
		, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

TSharedRef<SWidget> FEaseCurveEditorExtension::GetToolbarButtonMenuContent()
{
	const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance();
	if (!ToolInstance.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, ToolInstance->GetCommandList());

	const FName ToolId = FEaseCurveToolExtender::GetToolInstanceId(*ToolInstance);
	const TSharedRef<SWidget> QuickPresetMenuWidget = FEaseCurveToolExtender::MakeQuickPresetMenu(ToolId);
	MenuBuilder.AddWidget(QuickPresetMenuWidget, FText::GetEmpty());

	return MenuBuilder.MakeWidget();
}

EVisibility FEaseCurveEditorExtension::GetToolbarButtonVisibility() const
{
	if (const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>())
	{
		return ToolSettings->ShouldShowCurveEditorToolbar() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

TSharedPtr<FEaseCurveTool> FEaseCurveEditorExtension::GetToolInstance() const
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
	{
		return FEaseCurveToolExtender::Get().FindToolInstanceByCurveEditor(CurveEditor.ToSharedRef());
	}
	return nullptr;
}

bool FEaseCurveEditorExtension::IsButtonEnabled() const
{
	if (const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance())
	{
		// In order to apply operations on keys other than cubic weighted, we skip checking
		// ToolInstance->GetSelectionError() == EEaseCurveToolError::None
		return ToolInstance->IsCurveEditorSelection()
			&& ToolInstance->HasCachedKeysToEase();
	}
	return false;
}

FText FEaseCurveEditorExtension::GetEaseCurveTooltipText() const
{
	static const FText DefaultTooltipText = LOCTEXT("EaseCurveTooltip", "Apply an ease curve between the selected key tangents");

	const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance();
	if (!ToolInstance.IsValid())
	{
		return FText::Format(LOCTEXT("NoToolInstanceTooltip", "{0}\n\nError: No tool instance!")
			, DefaultTooltipText);
	}

	if (!ToolInstance->IsCurveEditorSelection()
		|| !ToolInstance->HasCachedKeysToEase())
	{
		return FText::Format(LOCTEXT("NoSelectionTooltip", "{0}\n\nError: No selection!")
			, DefaultTooltipText);
	}

	if (ToolInstance->GetSelectionError() != EEaseCurveToolError::None)
	{
		return FText::Format(LOCTEXT("NoEaseableKeysTooltip", "{0}\n\nError: {1}")
			, DefaultTooltipText, ToolInstance->GetSelectionErrorText());
	}

	return DefaultTooltipText;
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
