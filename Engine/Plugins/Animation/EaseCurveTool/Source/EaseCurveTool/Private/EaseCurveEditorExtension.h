// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorExtension.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class FExtender;
class FToolBarBuilder;
class SWidget;
struct EVisibility;
struct FEaseCurvePreset;
struct FFrameRate;

namespace UE::EaseCurveTool
{

class FCurveEditorToolBarMenu;
class FEaseCurveTool;

class FEaseCurveEditorExtension : public ICurveEditorExtension
{
public:
	explicit FEaseCurveEditorExtension(const TWeakPtr<FCurveEditor>& InWeakCurveEditor);

	//~ Begin ICurveEditorExtension
	virtual void BindCommands(TSharedRef<FUICommandList> InCommandList) override;
	virtual TSharedPtr<FExtender> MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End ICurveEditorExtension

private:
	void ExtendCurveEditorToolbar(FToolBarBuilder& ToolBarBuilder);

	void PopupToolbarMenu();

	TSharedRef<SWidget> GetToolbarButtonMenuContent();

	EVisibility GetToolbarButtonVisibility() const;

	TSharedPtr<FEaseCurveTool> GetToolInstance() const;

	bool IsButtonEnabled() const;

	FText GetEaseCurveTooltipText() const;

	TWeakPtr<FCurveEditor> WeakCurveEditor;

	TSharedPtr<FCurveEditorToolBarMenu> ButtonMenu;
};

} // namespace UE::CurveEditorTools
