// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveKeySelection.h"
#include "Channels/ChannelCurveModel.h"
#include "Channels/DoubleChannelCurveModel.h"
#include "Channels/FloatChannelCurveModel.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "CurveEditor.h"
#include "EaseCurveTangents.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolUtils.h"
#include "IKeyArea.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ScopedTransaction.h"

using namespace UE::Sequencer;

namespace UE::EaseCurveTool
{

template <class InChannelType, class InChannelValue>
void NormalizeChannelValues(const FKeyHandle& InKeyHandle, const FKeyHandle& InNextKeyHandle
	, const FMovieSceneChannelHandle& InChannelHandle
	, const bool bInAutoFlipTangents
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution
	, TArray<FEaseCurveTangents>& OutKeySetTangents
	, TArray<FEaseCurveTangents>& OutChangingTangents)
{
	TMovieSceneChannelHandle<InChannelType> Channel = InChannelHandle.Cast<InChannelType>();
	TMovieSceneChannelData<InChannelValue> ChannelData = Channel.Get()->GetData();

	const TArrayView<InChannelValue> ChannelValues = ChannelData.GetValues();
	const TArrayView<FFrameNumber> ChannelTimes = ChannelData.GetTimes();

	const int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
	const int32 NextKeyIndex = ChannelData.GetIndex(InNextKeyHandle);

	// If there is a key frame after this key frame that we are editing, we check if the that key frame value is less
	// than or greater than this key frame value. If less, flip the tangent (if option is set).
	const bool bIncreasingValue = ChannelValues[NextKeyIndex].Value >= ChannelValues[KeyIndex].Value;

	FEaseCurveTangents Tangents = FEaseCurveTangents(ChannelValues[KeyIndex], ChannelValues[NextKeyIndex]);

	if (bInAutoFlipTangents && !bIncreasingValue)
	{
		Tangents.Start *= -1.f;
		Tangents.End *= -1.f;
	}

	// Scale time/value to normalized tangent range
	FEaseCurveTangents ScaledTangents = Tangents;
	ScaledTangents.Normalize(ChannelTimes[KeyIndex], ChannelValues[KeyIndex].Value
		, ChannelTimes[NextKeyIndex], ChannelValues[NextKeyIndex].Value
		, InDisplayRate, InTickResolution);

	OutKeySetTangents.Add(ScaledTangents);
	if (ChannelValues[KeyIndex].Value != ChannelValues[NextKeyIndex].Value)
	{
		OutChangingTangents.Add(ScaledTangents);
	}
}

template <class InChannelType, class InValueType>
void SetChannelValues(const FEaseCurveTangents& InTangents
	, const EEaseCurveToolOperation InOperation
	, const FKeyHandle& InKeyHandle, const FKeyHandle& InNextKeyHandle
	, const FMovieSceneChannelHandle& InChannelHandle
	, const bool bInAutoFlipTangents
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution
	, const TObjectPtr<UMovieSceneSection>& InSection)
{
	TMovieSceneChannelHandle<InChannelType> Channel = /*InChannelData.ChannelHandle*/InChannelHandle.Cast<InChannelType>();
	TMovieSceneChannelData<InValueType> ChannelData = Channel.Get()->GetData();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannel;
	TArrayView<InValueType> ChannelValues = ChannelData.GetValues();
	const TArrayView<FFrameNumber> ChannelTimes = ChannelData.GetTimes();

	const int32 KeyIndex = ChannelData.GetIndex(InKeyHandle);
	const int32 NextKeyIndex = ChannelData.GetIndex(InNextKeyHandle);

	FEaseCurveTangents ScaledTangents = InTangents;

	// If there is a key frame after this key frame that we are editing, we check if the that key frame value is less
	// than or greater than this key frame value. If less, flip the tangent (if option is set).
	const bool bIncreasingValue = ChannelValues[NextKeyIndex].Value >= ChannelValues[KeyIndex].Value;

	if (bInAutoFlipTangents && !bIncreasingValue)
	{
		ScaledTangents.Start *= -1.f;
		ScaledTangents.End *= -1.f;
	}

	// Scale normalized tangents to time/value range
	ScaledTangents.ScaleUp(ChannelTimes[KeyIndex], ChannelValues[KeyIndex].Value
		, ChannelTimes[NextKeyIndex], ChannelValues[NextKeyIndex].Value
		, InDisplayRate, InTickResolution);

	const FScopedTransaction Transaction(NSLOCTEXT("EaseCurveTool", "SetSequencerCurveTangents", "Set Sequencer Curve Tangents"));

	// Mark the section as changed to force the curve editor to update its display
	if (InSection)
	{
		InSection->Modify();
		InSection->MarkAsChanged();
	}

	// Set this keys leave tangent
	if (InOperation == EEaseCurveToolOperation::Out || InOperation == EEaseCurveToolOperation::InOut)
	{
		ChannelValues[KeyIndex].InterpMode = ERichCurveInterpMode::RCIM_Cubic;
		ChannelValues[KeyIndex].Tangent.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedBoth;
		ChannelValues[KeyIndex].TangentMode = ERichCurveTangentMode::RCTM_Break;
		ChannelValues[KeyIndex].Tangent.LeaveTangent = ScaledTangents.Start;
		ChannelValues[KeyIndex].Tangent.LeaveTangentWeight = ScaledTangents.StartWeight;
	}

	// Set the next keys arrive tangent
	if (NextKeyIndex != INDEX_NONE && (InOperation == EEaseCurveToolOperation::In || InOperation == EEaseCurveToolOperation::InOut))
	{
		ChannelValues[NextKeyIndex].InterpMode = ERichCurveInterpMode::RCIM_Cubic;
		ChannelValues[NextKeyIndex].Tangent.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedBoth;
		ChannelValues[NextKeyIndex].TangentMode = ERichCurveTangentMode::RCTM_Break;
		ChannelValues[NextKeyIndex].Tangent.ArriveTangent = ScaledTangents.End;
		ChannelValues[NextKeyIndex].Tangent.ArriveTangentWeight = ScaledTangents.EndWeight;
	}
}

FEaseCurveKeySelection::FEaseCurveKeySelection(const TSharedPtr<ISequencer>& InSequencer)
{
	if (!InSequencer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FCurveEditor> CurveEditor = FEaseCurveToolUtils::GetCurveEditorFromSequencer(InSequencer.ToSharedRef()))
	{
		const FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
		const TMap<FCurveModelID, FKeyHandleSet> SelectedKeys = CurveEditorSelection.GetAll();
		if (!SelectedKeys.IsEmpty())
		{
			const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& Curves = CurveEditor->GetCurves();

			for (const TPair<FCurveModelID, FKeyHandleSet>& SelectedKeyPair : SelectedKeys)
			{
				const TUniquePtr<FCurveModel>* CurveModelUniquePtr = Curves.Find(SelectedKeyPair.Key);
				if (!CurveModelUniquePtr)
				{
					continue;
				}

				FCurveModel& CurveModel = *CurveModelUniquePtr->Get();

				bool bContinueProcessing = true;

				FMovieSceneChannelHandle ChannelHandle;
				if (FEaseCurveToolUtils::FindChannelHandleFromCurveModel(CurveModel, ChannelHandle))
				{
					for (const FKeyHandle& SelectedKey : SelectedKeyPair.Value.AsArray())
					{
						if (!ProcessCurveEditorKeySelection(CurveModel, ChannelHandle, SelectedKey))
						{
							bContinueProcessing = false;
							break;
						}
					}
				}

				if (!bContinueProcessing)
				{
					break;
				}
			}
		}
	}

	// Fallback to sequencer key selection if available
	if (SelectionError == EEaseCurveToolError::None && CurveEditorKeyData.IsEmpty())
	{
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = InSequencer->GetViewModel();
		if (!SequencerViewModel.IsValid())
		{
			return;
		}

		const TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel->GetSelection();
		if (!SequencerSelection.IsValid())
		{
			return;
		}

		for (const FKeyHandle& Key : SequencerSelection->KeySelection)
		{
			if (!ProcessSequencerKeySelection(SequencerSelection, Key))
			{
				break;
			}
		}
	}

	SetSelectionError();
}

void FEaseCurveKeySelection::SetSelectionError()
{
	bool bAllLast = true;
	bool bAllWeightedBroken = true;
	bool bAllSameValue = true;
	bool bAnySameValue = false;

	auto LoopKeyHandles = [&bAllLast, &bAllWeightedBroken, &bAllSameValue, &bAnySameValue](const TMap<FKeyHandle, EEaseCurveToolError>& InKeyErrors)
	{
		for (const TPair<FKeyHandle, EEaseCurveToolError>& KeyHandlePair : InKeyErrors)
		{
			if (EnumHasAnyFlags(KeyHandlePair.Value, EEaseCurveToolError::SameValues))
			{
				bAnySameValue = true;
			}
			else
			{
				bAllSameValue = false;
			}
			if (!EnumHasAnyFlags(KeyHandlePair.Value, EEaseCurveToolError::LastKey))
			{
				bAllLast = false;
			}
			if (EnumHasAnyFlags(KeyHandlePair.Value, EEaseCurveToolError::NoWeightedBrokenCubicTangents))
			{
				bAllWeightedBroken = false;
			}
		}
	};

	if (CurveEditorKeyData.Num() > 0)
	{
		for (const TPair<FCurveModelID, FEaseCurveChannelKeyCache>& KeyDataPair : CurveEditorKeyData)
		{
			LoopKeyHandles(KeyDataPair.Value.KeyHandles);
		}
	}
	else
	{
		for (const auto& [ChannelModel, KeyCache] : SequencerKeyData)
		{
			LoopKeyHandles(KeyCache.KeyHandles);
		}
	}

	if (bAllSameValue)
	{
		if (bAllLast)
		{
			SelectionError = EEaseCurveToolError::LastKey;
		}
		else
		{
			SelectionError = EEaseCurveToolError::SameValues;
		}
	}
	else if (bAllLast)
	{
		if (bAnySameValue)
		{
			SelectionError = EEaseCurveToolError::SameValues;
		}
		else
		{
			SelectionError = EEaseCurveToolError::LastKey;
		}
	}
	else if (!bAllWeightedBroken)
	{
		SelectionError = EEaseCurveToolError::NoWeightedBrokenCubicTangents;
	}
}

bool FEaseCurveKeySelection::ProcessSequencerKeySelection(const TSharedPtr<FSequencerSelection>& InSequencerSelection, const FKeyHandle InKey)
{
	if (!InSequencerSelection.IsValid() || InKey == FKeyHandle::Invalid())
	{
		return false;
	}

	const TViewModelPtr<FChannelModel> ChannelModel = InSequencerSelection->KeySelection.GetModelForKey(InKey);
	if (!ChannelModel.IsValid())
	{
		return false;
	}

	const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
	if (!KeyArea.IsValid())
	{
		return false;
	}

	const FMovieSceneChannelHandle& ChannelHandle = KeyArea->GetChannel();
	FEaseCurveChannelKeyCache& Entry = SequencerKeyData.FindOrAdd(ChannelModel);
	Entry.Section = ChannelModel->GetSection();
	Entry.ChannelHandle = ChannelHandle;
	Entry.KeyHandles.Add(InKey, FindErrorForKey(ChannelHandle, InKey));

	TotalSelectedKeys++;

	return true;
}

bool FEaseCurveKeySelection::ProcessCurveEditorKeySelection(FCurveModel& InCurveModel
	, const FMovieSceneChannelHandle& InChannelHandle, const FKeyHandle& InKey)
{
	if (!InChannelHandle.Get())
	{
		return false;
	}

	const FCurveModelID CurveId = InCurveModel.GetId().GetValue();

	FEaseCurveChannelKeyCache& Entry = CurveEditorKeyData.FindOrAdd(CurveId);
	Entry.Section = InCurveModel.GetOwningObjectOrOuter<UMovieSceneSection>();
	Entry.ChannelHandle = InChannelHandle;
	Entry.KeyHandles.Add(InKey, FindErrorForKey(InChannelHandle, InKey));

	TotalSelectedKeys++;

	return true;
}

EEaseCurveToolError FEaseCurveKeySelection::FindErrorForKey(const FMovieSceneChannelHandle& InChannelHandle, const FKeyHandle& InKey)
{
	const FName ChannelTypeName = InChannelHandle.GetChannelTypeName();

	if (ChannelTypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
	{
		return FindErrorForKey_Internal<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>(InKey, InChannelHandle);
	}
	if (ChannelTypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
	{
		return FindErrorForKey_Internal<FMovieSceneFloatChannel, FMovieSceneFloatValue>(InKey, InChannelHandle);
	}

	return EEaseCurveToolError::None;
}

void FEaseCurveKeySelection::ForEachEaseableKey(const FProcessKeySignature& InCallable)
{
	if (!CurveEditorKeyData.IsEmpty())
	{
		for (const TPair<FCurveModelID, FEaseCurveChannelKeyCache>& Entry : CurveEditorKeyData)
		{
			if (!ForEachChannelKey(Entry.Value.ChannelHandle
				, Entry.Value.KeyHandles, Entry.Value.Section, InCallable))
			{
				return;
			}
		}

		// Prioritize curve editor key selections or sequencer key selections
		return;
	}

	for (const auto& [ChannelModel, KeyCache] : SequencerKeyData)
	{
		if (!ForEachChannelKey(KeyCache.ChannelHandle, KeyCache.KeyHandles, KeyCache.Section, InCallable))
		{
			return;
		}
	}
}

bool FEaseCurveKeySelection::ForEachChannelKey(const FMovieSceneChannelHandle& InChannelHandle
	, const TMap<FKeyHandle, EEaseCurveToolError>& InKeyHandles
	, const TObjectPtr<UMovieSceneSection>& InSection
	, const FProcessKeySignature& InCallable)
{
	const FName ChannelTypeName = InChannelHandle.GetChannelTypeName();

	if (ChannelTypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
	{
		return ForEachChannelKey_Internal<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>(InChannelHandle
			, InKeyHandles, InSection, InCallable);
	}
	if (ChannelTypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
	{
		return ForEachChannelKey_Internal<FMovieSceneFloatChannel, FMovieSceneFloatValue>(InChannelHandle
			, InKeyHandles, InSection, InCallable);
	}

	return true;
}

FEaseCurveTangents FEaseCurveKeySelection::AverageTangents(const FFrameRate& InDisplayRate
	, const FFrameRate& InTickResolution
	, const bool bInAutoFlipTangents
	, const bool bInFirstOnly)
{
	TArray<FEaseCurveTangents> KeySetTangents;
	TArray<FEaseCurveTangents> ChangingTangents;

	ForEachEaseableKey([&InDisplayRate, &InTickResolution, bInAutoFlipTangents, bInFirstOnly, &KeySetTangents, &ChangingTangents]
			(const FKeyHandle& InKeyHandle
			, const FKeyHandle& InNextKeyHandle
			, const FMovieSceneChannelHandle& InChannelHandle
			, const TObjectPtr<UMovieSceneSection>& InSection)
		{
			const FName ChannelTypeName = InChannelHandle.GetChannelTypeName();
			if (ChannelTypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
			{
				NormalizeChannelValues<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>(InKeyHandle, InNextKeyHandle, InChannelHandle
					, bInAutoFlipTangents, InDisplayRate, InTickResolution, KeySetTangents, ChangingTangents);
			}
			else if (ChannelTypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
			{
				NormalizeChannelValues<FMovieSceneFloatChannel, FMovieSceneFloatValue>(InKeyHandle, InNextKeyHandle, InChannelHandle
					, bInAutoFlipTangents, InDisplayRate, InTickResolution, KeySetTangents, ChangingTangents);
			}

			return !bInFirstOnly;
		});

	return FEaseCurveTangents::Average(ChangingTangents);
}

void FEaseCurveKeySelection::SetTangents(const FEaseCurveTangents& InTangents
	, const EEaseCurveToolOperation InOperation
	, const FFrameRate& InDisplayRate
	, const FFrameRate& InTickResolution
	, const bool bInAutoFlipTangents)
{
	if (TotalSelectedKeys == 0)
	{
		return;
	}

	ForEachEaseableKey([&InDisplayRate, &InTickResolution, &InTangents, InOperation, bInAutoFlipTangents]
			(const FKeyHandle& InKeyHandle
			, const FKeyHandle& InNextKeyHandle
			, const FMovieSceneChannelHandle& InChannelHandle
			, const TObjectPtr<UMovieSceneSection>& InSection)
		{
			const FName ChannelTypeName = InChannelHandle.GetChannelTypeName();
			if (ChannelTypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
			{
				SetChannelValues<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>(InTangents
					, InOperation, InKeyHandle, InNextKeyHandle, InChannelHandle
					, bInAutoFlipTangents, InDisplayRate, InTickResolution, InSection);
			}
			else if (ChannelTypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
			{
				SetChannelValues<FMovieSceneFloatChannel, FMovieSceneFloatValue>(InTangents
					, InOperation, InKeyHandle, InNextKeyHandle, InChannelHandle
					, bInAutoFlipTangents, InDisplayRate, InTickResolution, InSection);
			}

			return true;
		});
}

int32 FEaseCurveKeySelection::GetKeyChannelCount() const
{
	const int32 CurveEditorKeyCount = CurveEditorKeyData.Num();
	if (CurveEditorKeyCount > 0)
	{
		return CurveEditorKeyCount;
	}

	const int32 SequencerKeyCount = SequencerKeyData.Num();
	if (SequencerKeyCount > 0)
	{
		return SequencerKeyCount;
	}

	return 0;
}

} // namespace UE::EaseCurveTool
