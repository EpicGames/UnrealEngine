// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "EaseCurveToolUtils.h"
#include "MVVM/ViewModelPtr.h"
#include "EaseCurveKeySelection.generated.h"

class FCurveModel;
class ISequencer;
class UMovieSceneSection;
struct FEaseCurveTangents;
struct FKeyHandle;
struct FMovieSceneChannelHandle;

namespace UE::Sequencer
{
	class FChannelModel;
	class FSequencerSelection;
}

UENUM()
enum class EEaseCurveToolError : uint8
{
	/** All selected keys are valid ease-able keys */
	None = 0,
	/** All selected keys are the last key of the channel and have no next key to ease to */
	LastKey = 1 << 1,
	/** All selected keys have the same values (easing would result in no change) */
	SameValues = 1 << 2,
	/** All selected keys are not weighted, broken, cubic tangents */
	NoWeightedBrokenCubicTangents = 1 << 3
};
ENUM_CLASS_FLAGS(EEaseCurveToolError)

namespace UE::EaseCurveTool
{

enum class EEaseCurveToolOperation : uint8;

/** Cached set of key handles for a channel */
struct FEaseCurveChannelKeyCache
{
	TObjectPtr<UMovieSceneSection> Section;
	FMovieSceneChannelHandle ChannelHandle;
	TMap<FKeyHandle, EEaseCurveToolError> KeyHandles;

	template <class InChannelHandleType, class InChannelValueType>
	TMovieSceneChannelData<InChannelValueType> GetChannelData()
	{
		const TMovieSceneChannelHandle<InChannelHandleType> Channel = ChannelHandle.Cast<InChannelHandleType>();
		return Channel.Get()->GetData();
	}
};

/** Represents a key selection, either Curve Editor (prioritized) or Sequencer, that can be operated on by the Ease Curve Tool */
struct FEaseCurveKeySelection
{
	/** Constructs an empty ease curve key selection */
	FEaseCurveKeySelection() {}
	/** Constructs an ease curve key selection from a sequencer key selection */
	FEaseCurveKeySelection(const TSharedPtr<ISequencer>& InSequencer);

	FEaseCurveTangents AverageTangents(const FFrameRate& InDisplayRate
		, const FFrameRate& InTickResolution
		, const bool bInAutoFlipTangents
		, const bool bInFirstOnly = false);

	void SetTangents(const FEaseCurveTangents& InTangents
		, const EEaseCurveToolOperation InOperation
		, const FFrameRate& InDisplayRate
		, const FFrameRate& InTickResolution
		, const bool bInAutoFlipTangents);

	bool HasCachedKeysToEase() const
	{
		return !CurveEditorKeyData.IsEmpty() || !SequencerKeyData.IsEmpty();
	}

	int32 GetTotalSelectedKeys() const
	{
		return TotalSelectedKeys;
	}

	bool IsCurveEditorSelection() const
	{
		return !CurveEditorKeyData.IsEmpty();
	}

	int32 GetKeyChannelCount() const;

	/** @return Error code specifying why the key selection is not valid to be used by the Ease Curve Tool to perform operations on */
	EEaseCurveToolError GetSelectionError() const
	{
		return SelectionError;
	}

protected:
	using FProcessKeySignature = TFunctionRef<bool(const FKeyHandle& /*InKeyHandle*/
		, const FKeyHandle& /*InNextKeyHandle*/
		, const FMovieSceneChannelHandle& /*InChannelHandle*/
		, const TObjectPtr<UMovieSceneSection>& /*InSection*/)>;

	void SetSelectionError();

	EEaseCurveToolError FindErrorForKey(const FMovieSceneChannelHandle& InChannelHandle, const FKeyHandle& InKey);

	template <class InChannelType, class InValueType>
	static EEaseCurveToolError FindErrorForKey_Internal(const FKeyHandle& InKeyHandle, const FMovieSceneChannelHandle& InChannelHandle)
	{
		EEaseCurveToolError OutError = EEaseCurveToolError::None;

		InChannelType* const Channel = static_cast<InChannelType*>(InChannelHandle.Get());
		if (!Channel)
		{
			return OutError;
		}

		const int32 KeyIndex = Channel->GetIndex(InKeyHandle);
		if (KeyIndex == INDEX_NONE)
		{
			return OutError;
		}

		const TArrayView<const InValueType> ChannelValues = Channel->GetValues();
		if (!ChannelValues.IsValidIndex(KeyIndex))
		{
			return OutError;
		}

		const int32 NextKeyIndex = KeyIndex + 1;
		if (ChannelValues.IsValidIndex(NextKeyIndex))
		{
			if (ChannelValues[KeyIndex].Value == ChannelValues[NextKeyIndex].Value)
			{
				OutError |= EEaseCurveToolError::SameValues;
			}
		}
		else
		{
			OutError |= EEaseCurveToolError::LastKey;
		}

		if (!FEaseCurveToolUtils::HasWeightedBrokenTangents(ChannelValues[KeyIndex]))
		{
			OutError |= EEaseCurveToolError::NoWeightedBrokenCubicTangents;
		}

		return OutError;
	}

	bool ProcessSequencerKeySelection(const TSharedPtr<Sequencer::FSequencerSelection>& InSequencerSelection, const FKeyHandle InKey);
	bool ProcessCurveEditorKeySelection(FCurveModel& CurveModel, const FMovieSceneChannelHandle& ChannelHandle, const FKeyHandle& InKey);

	void ForEachEaseableKey(const FProcessKeySignature& InCallable);

	bool ForEachChannelKey(const FMovieSceneChannelHandle& InChannelHandle
		, const TMap<FKeyHandle, EEaseCurveToolError>& InKeyHandles
		, const TObjectPtr<UMovieSceneSection>& InSection
		, const FProcessKeySignature& InCallable);

	template <class InChannelType, class InValueType>
	bool ForEachChannelKey_Internal(const FMovieSceneChannelHandle& InChannelHandle
		, const TMap<FKeyHandle, EEaseCurveToolError>& InKeyHandles
		, const TObjectPtr<UMovieSceneSection>& InSection
		, const FProcessKeySignature& InCallable)
	{
		if (!InChannelHandle.Get())
		{
			return false;
		}

		TMovieSceneChannelHandle<InChannelType> Channel = InChannelHandle.Cast<InChannelType>();
		TMovieSceneChannelData<InValueType> ChannelData = Channel.Get()->GetData();

		const TArrayView<InValueType> ChannelValues = ChannelData.GetValues();
		const int32 KeyCount = ChannelValues.Num();

		for (const TPair<FKeyHandle, EEaseCurveToolError>& KeyHandlePair : InKeyHandles)
		{
			if (KeyHandlePair.Key == FKeyHandle::Invalid())
			{
				continue;
			}

			const int32 KeyIndex = ChannelData.GetIndex(KeyHandlePair.Key);
			if (KeyIndex == INDEX_NONE)
			{
				continue;
			}

			// If there is no key after the selected key, we don't need to process.
			// The arrive tangents of this key will be set by the previous key's processing.
			int32 NextKeyIndex = KeyIndex + 1;
			NextKeyIndex = (NextKeyIndex < KeyCount) ? NextKeyIndex : INDEX_NONE;
			if (NextKeyIndex == INDEX_NONE)
			{
				continue;
			}

			// Need to check if the next key index is valid, otherwise GetHandle() will fail.
			const FKeyHandle NextKeyHandle = ChannelData.GetHandle(NextKeyIndex);
			if (NextKeyHandle == FKeyHandle::Invalid())
			{
				continue;
			}

			if (!InCallable(KeyHandlePair.Key, NextKeyHandle, InChannelHandle, InSection))
			{
				return false;
			}
		}

		return true;
	}

	TMap<UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>, FEaseCurveChannelKeyCache> SequencerKeyData;
	TMap<FCurveModelID, FEaseCurveChannelKeyCache> CurveEditorKeyData;

	int32 TotalSelectedKeys = 0;

	EEaseCurveToolError SelectionError = EEaseCurveToolError::None;
};

} // namespace UE::EaseCurveTool
