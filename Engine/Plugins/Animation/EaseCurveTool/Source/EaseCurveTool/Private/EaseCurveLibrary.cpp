// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveLibrary.h"
#include "EaseCurveToolSettings.h"
#include "ScopedTransaction.h"

DEFINE_LOG_CATEGORY_STATIC(LogEaseCurveLibrary, Log, All);

#define LOCTEXT_NAMESPACE "EaseCurveLibrary"

TArray<FEaseCurvePreset> UEaseCurveLibrary::GetDefaultPresets()
{
	TArray<FEaseCurvePreset> DefaultPresets;

	static const FText EaseInOutCategory = LOCTEXT("EaseInOut", "Ease In Out");
	static const FText EaseInCategory = LOCTEXT("EaseIn", "Ease In");
	static const FText EaseOutCategory = LOCTEXT("EaseOut", "Ease Out");

	static const FText SinePreset = LOCTEXT("Sine", "Sine");
	static const FText CubicPreset = LOCTEXT("Cubic", "Cubic");
	static const FText QuinticPreset = LOCTEXT("Quintic", "Quintic");
	static const FText CircularPreset = LOCTEXT("Circular", "Circular");
	static const FText QuadraticPreset = LOCTEXT("Quadratic", "Quadratic");
	static const FText QuarticPreset = LOCTEXT("Quartic", "Quartic");
	static const FText ExponentialPreset = LOCTEXT("Exponential", "Exponential");
	static const FText BackPreset = LOCTEXT("Back", "Back");

	auto AddPreset = [&DefaultPresets](const FText& InName
		, const FText& InCategory, const TStaticArray<double, 4>& InPoints)
	{
		DefaultPresets.Add(FEaseCurvePreset(InCategory, InName
			, FEaseCurveTangents::MakeFromCubicBezier(InPoints)));
	};

	/**
	 * Standard easing cubic bezier presets taken from www.easings.net
	 * in the form of: { StartX, StartY, EndX, EndY }
	 */
	// Category: Ease In Out
	AddPreset(SinePreset,        EaseInOutCategory, { 0.37,  0.0, 0.63, 1.0 });
	AddPreset(CubicPreset,       EaseInOutCategory, { 0.65,  0.0, 0.35, 1.0 });
	AddPreset(QuinticPreset,     EaseInOutCategory, { 0.83,  0.0, 0.17, 1.0 });
	AddPreset(CircularPreset,    EaseInOutCategory, { 0.85,  0.0, 0.15, 1.0 });
	AddPreset(QuadraticPreset,   EaseInOutCategory, { 0.45,  0.0, 0.55, 1.0 });
	AddPreset(QuarticPreset,     EaseInOutCategory, { 0.76,  0.0, 0.24, 1.0 });
	AddPreset(ExponentialPreset, EaseInOutCategory, { 0.87,  0.0, 0.13, 1.0 });
	AddPreset(BackPreset,        EaseInOutCategory, { 0.68, -0.6, 0.32, 1.6 });
	// Category: Ease In
	AddPreset(SinePreset,        EaseInCategory, { 0.12,  0.0,  0.39, 0.0 });
	AddPreset(CubicPreset,       EaseInCategory, { 0.32,  0.0,  0.67, 0.0 });
	AddPreset(QuinticPreset,     EaseInCategory, { 0.64,  0.0,  0.78, 0.0 });
	AddPreset(CircularPreset,    EaseInCategory, { 0.55,  0.0,  1.0,  0.45 });
	AddPreset(QuadraticPreset,   EaseInCategory, { 0.11,  0.0,  0.5,  0.0 });
	AddPreset(QuarticPreset,     EaseInCategory, { 0.5,   0.0,  0.75, 0.0 });
	AddPreset(ExponentialPreset, EaseInCategory, { 0.7,   0.0,  0.84, 0.0 });
	AddPreset(BackPreset,        EaseInCategory, { 0.36, -0.56, 0.66, 0.0 });
	// Category: Ease Out
	AddPreset(SinePreset,        EaseOutCategory, { 0.61, 1.0,  0.88, 1.0 });
	AddPreset(CubicPreset,       EaseOutCategory, { 0.33, 1.0,  0.68, 1.0 });
	AddPreset(QuinticPreset,     EaseOutCategory, { 0.22, 1.0,  0.36, 1.0 });
	AddPreset(CircularPreset,    EaseOutCategory, { 0.0,  0.55, 0.45, 1.0 });
	AddPreset(QuadraticPreset,   EaseOutCategory, { 0.5,  1.0,  0.89, 1.0 });
	AddPreset(QuarticPreset,     EaseOutCategory, { 0.25, 1.0,  0.5,  1.0 });
	AddPreset(ExponentialPreset, EaseOutCategory, { 0.16, 1.0,  0.3,  1.0 });
	AddPreset(BackPreset,        EaseOutCategory, { 0.34, 1.56, 0.64, 1.0 });

	return DefaultPresets;
}

bool UEaseCurveLibrary::DoesPresetExist(const FEaseCurvePresetHandle& InPresetHandle) const
{
	return Presets.ContainsByPredicate([&InPresetHandle](const FEaseCurvePreset& InPreset)
		{
			return InPreset == InPresetHandle;
		});
}

const TArray<FEaseCurvePreset>& UEaseCurveLibrary::GetPresets() const
{
	return Presets;
}

TArray<FText> UEaseCurveLibrary::GetCategories() const
{
	TArray<FText> OutCategories;

	for (const FEaseCurvePreset& Preset : Presets)
	{
		bool bFound = false;
		for (const FText& ExistingCategory : OutCategories)
		{
			if (Preset.Category.EqualToCaseIgnored(ExistingCategory))
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			OutCategories.Add(Preset.Category);
		}
	}

	OutCategories.Append(EmptyCategories);

	OutCategories.Sort([](const FText& InA, const FText& InB)
		{
			return InA.CompareToCaseIgnored(InB) < 0;
		});

	return MoveTemp(OutCategories);
}

TArray<FEaseCurvePreset> UEaseCurveLibrary::GetCategoryPresets(const FText& InCategory) const
{
	TArray<FEaseCurvePreset> OutPresets;

	for (const FEaseCurvePreset& Preset : Presets)
	{
		if (Preset.Category.EqualToCaseIgnored(InCategory))
		{
			OutPresets.Add(Preset);
		}
	}

	return MoveTemp(OutPresets);
}

bool UEaseCurveLibrary::AddPreset(const FEaseCurvePreset& InPreset)
{
	if (DoesPresetExist(InPreset.GetHandle()))
	{
		UE_LOGF(LogEaseCurveLibrary, Warning, "UEaseCurveLibrary::AddPreset() - Preset already exists: %ls"
			, *InPreset.Name.ToString());
		return false;
	}

	const int32 NumAdded = Presets.Add(InPreset);
	const bool bAdded = NumAdded > 0;

	if (bAdded)
	{
		PresetChangedDelegate.Broadcast();
	}

	return bAdded;
}

bool UEaseCurveLibrary::AddPresetToNewCategory(const FText& InName, const FEaseCurveTangents& InTangents, FEaseCurvePreset& OutNewPreset)
{
	const FText NewCategory = GetDefault<UEaseCurveToolSettings>()->GetNewPresetCategory();
	if (NewCategory.IsEmpty())
	{
		OutNewPreset = FEaseCurvePreset();
		return false;
	}

	OutNewPreset = FEaseCurvePreset(NewCategory, InName, InTangents);

	return AddPreset(OutNewPreset);
}

bool UEaseCurveLibrary::RemovePreset(const FEaseCurvePresetHandle& InPresetHandle)
{
	const int32 NumRemoved = Presets.RemoveAll([&InPresetHandle]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset == InPresetHandle;
		});

	const bool bRemoved = NumRemoved > 0;

	if (bRemoved)
	{
		PresetChangedDelegate.Broadcast();
	}
	
	return bRemoved;
}

bool UEaseCurveLibrary::ChangePresetCategory(const FEaseCurvePreset& InPreset, const FText& InNewCategory)
{
	FEaseCurvePreset* FoundPreset = Presets.FindByPredicate([&InPreset]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset.Category.EqualToCaseIgnored(InPreset.Category)
				&& InThisPreset.Name.EqualToCaseIgnored(InPreset.Name);
		});
	if (!FoundPreset)
	{
		return false;
	}

	FoundPreset->Category = InNewCategory;

	EmptyCategories.RemoveAll([&InNewCategory](const FText& InThisText)
		{
			return InThisText.EqualToCaseIgnored(InNewCategory);
		});

	return true;
}

bool UEaseCurveLibrary::FindPreset(const FEaseCurvePresetHandle& InPresetHandle, FEaseCurvePreset& OutPreset)
{
	const FEaseCurvePreset* const FoundPreset = Presets.FindByPredicate([&InPresetHandle]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset == InPresetHandle;
		});

	if (FoundPreset)
	{
		OutPreset = *FoundPreset;
		return true;
	}

	OutPreset = FEaseCurvePreset();
	return false;
}

bool UEaseCurveLibrary::FindPresetByTangents(const FEaseCurveTangents& InTangents
	, FEaseCurvePreset& OutPreset, const double InErrorTolerance)
{
	const FEaseCurvePreset* const FoundPreset = Presets.FindByPredicate([&InTangents, InErrorTolerance]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset.Tangents.IsNearlyEqual(InTangents, InErrorTolerance);
		});

	if (FoundPreset)
	{
		OutPreset = *FoundPreset;
		return true;
	}

	OutPreset = FEaseCurvePreset();
	return false;
}

bool UEaseCurveLibrary::DoesPresetCategoryExist(const FText& InCategory) const
{
	const bool bExists = Presets.ContainsByPredicate([&InCategory]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset.Category.EqualToCaseIgnored(InCategory);
		});
	if (bExists)
	{
		return true;
	}

	const FText* const FoundCategory = EmptyCategories.FindByPredicate([&InCategory](const FText& InThisText)
		{
			return InThisText.EqualToCaseIgnored(InCategory);
		});
	return FoundCategory != nullptr;
}

bool UEaseCurveLibrary::RenamePresetCategory(const FText& InCategory, const FText& InNewCategory)
{
	bool bAnyRenamed = false;

	for (FEaseCurvePreset& Preset : Presets)
	{
		if (Preset.Category.EqualToCaseIgnored(InCategory))
		{
			Preset.Category = InNewCategory;

			bAnyRenamed = true;
		}
	}

	EmptyCategories.RemoveAll([&InNewCategory](const FText& InThisText)
		{
			return InThisText.EqualToCaseIgnored(InNewCategory);
		});

	if (bAnyRenamed)
	{
		PresetChangedDelegate.Broadcast();
	}

	return bAnyRenamed;
}

FText UEaseCurveLibrary::GenerateNewEmptyCategory()
{
	static const FString DefaultNewCategoryName = TEXT("New Category");

	FText NewCategoryText;
	int32 CurrentNum = 1;

	while (true)
	{
		const FString NewCategoryString = (CurrentNum == 1)
			? DefaultNewCategoryName 
			: DefaultNewCategoryName + TEXT(" ") + FString::FromInt(CurrentNum);
		NewCategoryText = FText::FromString(NewCategoryString);

		if (!DoesPresetCategoryExist(NewCategoryText))
		{
			break;
		}

		CurrentNum++;
	}

	EmptyCategories.Add(NewCategoryText);

	return NewCategoryText;
}

bool UEaseCurveLibrary::RemovePresetCategory(const FText& InCategory)
{
	const int32 NumRemoved = Presets.RemoveAll([InCategory](const FEaseCurvePreset& InThisPreset)
		{
			return InCategory.EqualToCaseIgnored(InThisPreset.Category);
		});

	const int32 NumEmptyRemoved = EmptyCategories.RemoveAll([&InCategory](const FText& InThisText)
		{
			return InThisText.EqualToCaseIgnored(InCategory);
		});

	const bool bRemoved = NumRemoved > 0 || NumEmptyRemoved > 0;

	if (bRemoved)
	{
		PresetChangedDelegate.Broadcast();
	}

	return bRemoved;
}

bool UEaseCurveLibrary::RenamePreset(const FEaseCurvePresetHandle& InPresetHandle, const FText& InNewPresetName)
{
	FEaseCurvePreset* const FoundPreset = Presets.FindByPredicate([&InPresetHandle]
		(const FEaseCurvePreset& InThisPreset)
		{
			return InThisPreset == InPresetHandle;
		});
	if (!FoundPreset)
	{
		return false;
	}

	FoundPreset->Name = InNewPresetName;

	PresetChangedDelegate.Broadcast();

	return true;
}

void UEaseCurveLibrary::SetToDefaultPresets()
{
	const FScopedTransaction Transaction(LOCTEXT("SetToDefaultPresets", "Set To Default Presets"));

	Modify();
	
	Presets = GetDefaultPresets();
	Sort();

	PresetChangedDelegate.Broadcast();
}

void UEaseCurveLibrary::Sort()
{
	Presets.Sort([](const FEaseCurvePreset& InPresetA, const FEaseCurvePreset& InPresetB)
		{
			return InPresetA < InPresetB;
		});
}

#undef LOCTEXT_NAMESPACE
