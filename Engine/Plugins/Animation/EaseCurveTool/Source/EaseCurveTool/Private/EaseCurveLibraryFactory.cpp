// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveLibraryFactory.h"
#include "EaseCurveLibrary.h"

#define LOCTEXT_NAMESPACE "EaseCurveLibraryFactory"

UEaseCurveLibraryFactory::UEaseCurveLibraryFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UEaseCurveLibrary::StaticClass();
}

FText UEaseCurveLibraryFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Ease Curve Library");
}

FText UEaseCurveLibraryFactory::GetToolTip() const
{
	return LOCTEXT("Tooltip", "Data asset that holds category and tangent data for ease curve presets");
}

UObject* UEaseCurveLibraryFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UEaseCurveLibrary>(InParent, InClass, InName, Flags);
}

bool UEaseCurveLibraryFactory::ShouldShowInNewMenu() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
