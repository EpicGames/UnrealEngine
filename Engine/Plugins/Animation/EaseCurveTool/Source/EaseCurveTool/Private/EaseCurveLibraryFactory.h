// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "EaseCurveLibraryFactory.generated.h"

UCLASS()
class UEaseCurveLibraryFactory : public UFactory
{
	GENERATED_BODY()

public:
	UEaseCurveLibraryFactory();

	//~ Begin UFactory
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ End UFactory
};
