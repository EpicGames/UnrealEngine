// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "EaseCurveTangents.h"
#include "Templates/SharedPointer.h"
#include "EaseCurvePreset.generated.h"

/** Store category and name as a handle to pass around for preset identification */
struct FEaseCurvePresetHandle
{
	FEaseCurvePresetHandle() {}
	FEaseCurvePresetHandle(const FText& InCategory, const FText& InName)
		: Category(InCategory), Name(InName)
	{
		ensureMsgf(Category.IsEmpty() == false, TEXT("Category cannot be empty"));
		ensureMsgf(Name.IsEmpty() == false, TEXT("Name cannot be empty"));
	}

	FORCEINLINE bool operator==(const FEaseCurvePresetHandle& InRhs) const
	{
		return Name.EqualToCaseIgnored(InRhs.Name) && Category.EqualToCaseIgnored(InRhs.Category);
	}

	FText Category;
	FText Name;
};

USTRUCT()
struct FEaseCurvePreset
{
	GENERATED_BODY()

	FEaseCurvePreset() {}
	FEaseCurvePreset(const FText& InCategory, const FText& InName, const FEaseCurveTangents& InTangents)
		: Category(InCategory), Name(InName), Tangents(InTangents)
	{}
	FEaseCurvePreset(const FEaseCurvePresetHandle& InHandle, const FEaseCurveTangents& InTangents)
		: Category(InHandle.Category), Name(InHandle.Name), Tangents(InTangents)
	{}

	FORCEINLINE bool operator==(const FEaseCurvePresetHandle& InRhs) const
	{
		return GetHandle() == InRhs;
	}
	FORCEINLINE bool operator==(const FEaseCurvePreset& InRhs) const
	{
		return GetHandle() == InRhs.GetHandle();
	}
	FORCEINLINE bool operator!=(const FEaseCurvePreset& InRhs) const
	{
		return !(*this == InRhs);
	}

	FORCEINLINE bool operator<(const FEaseCurvePreset& InRhs) const
	{
		// Order from soft to hard based on curve length
		return Category.CompareToCaseIgnored(InRhs.Category) < 0
			&& Name.CompareToCaseIgnored(InRhs.Name) < 0
			&& Tangents.CalculateCurveLength() < InRhs.Tangents.CalculateCurveLength();
	}

	FEaseCurvePresetHandle GetHandle() const
	{
		return FEaseCurvePresetHandle(Category, Name);
	}

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurvePreset")
	FText Category;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurvePreset")
	FText Name;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurvePreset")
	FEaseCurveTangents Tangents;
};
