// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurvePresetDragDropOperation.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SEaseCurvePresetGroup.h"
#include "Widgets/SEaseCurvePresetGroupItem.h"

namespace UE::EaseCurveTool
{

FCursorReply FEaseCurvePresetDragDropOperation::OnCursorQuery()
{
	return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
}

TSharedPtr<SWidget> FEaseCurvePresetDragDropOperation::GetDefaultDecorator() const
{
	if (!WidgetWeak.IsValid())
	{
		return FDragDropOperation::GetDefaultDecorator();
	}

	return SNew(SBorder)
		.Padding(2.f)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Border")))
		.Content()
		[
			WidgetWeak.Pin().ToSharedRef()
		];
}

void FEaseCurvePresetDragDropOperation::OnDrop(bool bInDropWasHandled, const FPointerEvent& InMouseEvent)
{
	FDragDropOperation::OnDrop(bInDropWasHandled, InMouseEvent);

	if (const TSharedPtr<SEaseCurvePresetGroupItem> Widget = WidgetWeak.Pin())
	{
		Widget->TriggerEndMove();
	}

	for (const TWeakPtr<SEaseCurvePresetGroup>& GroupWidgetWeak : HoveredGroupWidgets)
	{
		if (const TSharedPtr<SEaseCurvePresetGroup> GroupWidget = GroupWidgetWeak.Pin())
		{
			GroupWidget->ResetDragBorder();
		}
	}
}

void FEaseCurvePresetDragDropOperation::AddHoveredGroup(const TSharedRef<SEaseCurvePresetGroup>& InGroupWidget)
{
	HoveredGroupWidgets.Add(InGroupWidget);
}

} // namespace UE::EaseCurveTool
