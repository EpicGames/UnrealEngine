// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "Templates/SharedPointer.h"

struct FEaseCurvePreset;

namespace UE::EaseCurveTool
{

class SEaseCurvePresetGroup;
class SEaseCurvePresetGroupItem;

class FEaseCurvePresetDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FEaseCurvePresetDragDropOperation, FDragDropOperation)

	FEaseCurvePresetDragDropOperation(const TSharedPtr<SEaseCurvePresetGroupItem>& InWidget, const TSharedPtr<FEaseCurvePreset>& InPreset)
		: WidgetWeak(InWidget), Preset(InPreset)
	{}

	TSharedPtr<SEaseCurvePresetGroupItem> GetWidget() const { return WidgetWeak.Pin(); }
	TSharedPtr<FEaseCurvePreset> GetPreset() const { return Preset; }

	//~ Begin FDragDropOperation
	virtual FCursorReply OnCursorQuery() override;
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual void OnDrop(bool bInDropWasHandled, const FPointerEvent& InMouseEvent) override;
	//~ End FDragDropOperation

	void AddHoveredGroup(const TSharedRef<SEaseCurvePresetGroup>& InGroupWidget);

protected:
	TWeakPtr<SEaseCurvePresetGroupItem> WidgetWeak;
	TSharedPtr<FEaseCurvePreset> Preset;

	TSet<TWeakPtr<SEaseCurvePresetGroup>> HoveredGroupWidgets;
};

} // namespace UE::EaseCurveTool
