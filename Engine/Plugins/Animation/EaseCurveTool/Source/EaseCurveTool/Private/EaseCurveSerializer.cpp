// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveSerializer.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "Internationalization/Text.h"
#include "Misc/Paths.h"
#include "EaseCurveLibrary.h"

#define LOCTEXT_NAMESPACE "EaseCurveSerializer"

bool UEaseCurveSerializer::PromptUserForFilePath(FString& OutFilePath, const bool bInImport)
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();
	if (!DesktopPlatform)
	{
		return false;
	}

	const void* const ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
	const FString Title = TEXT("Select File Location");
	const FString FileTypes = TEXT("All Files (*.*)|*.*");

	bool bSuccess;
	TArray<FString> OutFilePaths;

	if (bInImport)
	{
		bSuccess = DesktopPlatform->OpenFileDialog(ParentWindowHandle
			, Title
			, FPaths::GetPath(OutFilePath)
			, FPaths::GetBaseFilename(OutFilePath)
			, FileTypes
			, EFileDialogFlags::None
			, OutFilePaths);
	}
	else
	{
		bSuccess = DesktopPlatform->SaveFileDialog(ParentWindowHandle
			, Title
			, FPaths::GetPath(OutFilePath)
			, FPaths::GetBaseFilename(OutFilePath)
			, FileTypes
			, EFileDialogFlags::None
			, OutFilePaths);
	}

	if (!bSuccess || OutFilePaths.Num() != 1)
	{
		return false;
	}

	OutFilePath = OutFilePaths[0];

	return true;
}

FText UEaseCurveSerializer::GetDisplayName() const
{
	return FText::GetEmpty();
}

FText UEaseCurveSerializer::GetDisplayTooltip() const
{
	return FText::GetEmpty();
}

bool UEaseCurveSerializer::IsFileExport() const
{
	return true;
}

bool UEaseCurveSerializer::SupportsExport() const
{
	return false;
}

bool UEaseCurveSerializer::Export(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries)
{
	return false;
}

bool UEaseCurveSerializer::IsFileImport() const
{
	return true;
}

bool UEaseCurveSerializer::SupportsImport() const
{
	return false;
}

bool UEaseCurveSerializer::Import(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries)
{
	return false;
}

#undef LOCTEXT_NAMESPACE
