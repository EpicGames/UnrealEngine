// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

FEaseCurveStyle::FEaseCurveStyle()
	: FSlateStyleSet(TEXT("EaseCurveTool"))
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	FSlateStyleSet::SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	static const FVector2D Icon16x16(16.0f, 16.0f);
	static const FVector2D Icon20x20(20.0f, 20.0f);

	Set("Icon.ToolBar", new IMAGE_BRUSH_SVG("Icons/EaseCurveTool_16", Icon20x20));
	Set("Icon.Tab", new IMAGE_BRUSH_SVG("Icons/EaseCurveTool_16", Icon16x16));

	Set("Preset.Selected", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.f, 0.f, 0.f, 4.f), FStyleColors::Select.GetSpecifiedColor(), 1.f));

	Set("EditMode.Background", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FLinearColor(0.1f, 0.1f, 0.1f, 1.f), 1.f));
	Set("EditMode.Background.Highlight", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FLinearColor(0.6f, 0.6f, 0.6f, 1.f), 1.f));
	Set("EditMode.Background.Over", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::AccentBlue.GetSpecifiedColor(), 1.f));

	const FButtonStyle& SimpleButtonStyle = FAppStyle::GetWidgetStyle<FButtonStyle>(TEXT("SimpleButton"));

	constexpr float ToolButtonPadding = 2.f;
	Set("ToolButton.Padding", ToolButtonPadding);

	constexpr float ToolButtonImageSize = 12.f;
	Set("ToolButton.ImageSize", ToolButtonImageSize);

	const FButtonStyle ToolButtonStyle = FButtonStyle(SimpleButtonStyle)
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
		.SetNormalPadding(FMargin(ToolButtonPadding))
		.SetPressedPadding(FMargin(ToolButtonPadding, ToolButtonPadding + (ToolButtonPadding * 0.5f), ToolButtonPadding, ToolButtonPadding - (ToolButtonPadding * 0.5f)));
	Set("ToolButton", ToolButtonStyle);

	const FButtonStyle ToolButtonNoPadStyle = FButtonStyle(ToolButtonStyle)
		.SetNormalPadding(FMargin(0.f))
		.SetPressedPadding(FMargin(0.f));
	Set("ToolButton.NoPad", ToolButtonNoPadStyle);

	Set("ToolButton.Opaque", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f));

	const FCheckBoxStyle& ToggleButtonCheckboxStyle = FAppStyle::GetWidgetStyle<FCheckBoxStyle>(TEXT("ToggleButtonCheckbox"));

	const FCheckBoxStyle ToolToggleButtonStyle = FCheckBoxStyle(ToggleButtonCheckboxStyle)
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.f))
		.SetPadding(ToolButtonPadding);
	Set("ToolToggleButton", ToolToggleButtonStyle);

	Set("Editor.Border", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::Dropdown, 1.f));
	Set("Editor.Border.Hover", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::DropdownOutline, 1.f));

	Set("Editor.LabelFont", FSlateFontInfo(FCoreStyle::GetDefaultFont(), 7, TEXT("Regular")));
	Set("Editor.ErrorFont", FSlateFontInfo(FCoreStyle::GetDefaultFont(), 9, TEXT("Italic")));

	FButtonStyle MenuHoverHintOnly = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(), FStyleColors::Highlight.GetSpecifiedColor()))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(), FStyleColors::Highlight.GetSpecifiedColor()))
		.SetNormalPadding(FMargin(0.f,0.f,0.f,1.f))
		.SetPressedPadding(FMargin(0.f,1.f,0.f,0.f));
	Set("MenuHoverHintOnly", MenuHoverHintOnly);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FEaseCurveStyle::~FEaseCurveStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
