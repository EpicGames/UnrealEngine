// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FEaseCurveStyle final : public FSlateStyleSet
{
public:
	static FEaseCurveStyle& Get()
	{
		static FEaseCurveStyle Instance;
		return Instance;
	}

	FEaseCurveStyle();
	virtual ~FEaseCurveStyle() override;
};
