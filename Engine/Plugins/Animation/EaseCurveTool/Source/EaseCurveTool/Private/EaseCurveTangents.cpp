// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveTangents.h"
#include "Internationalization/Text.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"
#include "Widgets/SEaseCurveEditor.h"

#define LOCTEXT_NAMESPACE "EaseCurveTangents"

bool FEaseCurveTangents::FromString(const FString& InString, FEaseCurveTangents& OutTangents)
{
	const FString TangentsString = InString.Replace(TEXT(" "), TEXT(""));

	TArray<FString> ValueStrings;
	if (TangentsString.ParseIntoArray(ValueStrings, TEXT(",")) != 4)
	{
		return false;
	}

	// Convert strings to doubles
	TStaticArray<double, 4> Values = { 0.0, 0.0, 0.0, 0.0 };
	for (int32 Index = 0; Index < 4; ++Index)
	{
		if (!FDefaultValueHelper::ParseDouble(ValueStrings[Index], Values[Index]))
		{
			return false;
		}
	}

	// Convert four cubic bezier points to two points/tangents
	OutTangents.FromCubicBezier(Values);

	return true;
}

bool FEaseCurveTangents::CanParseString(const FString& InString)
{
	FEaseCurveTangents Tangents;
	return FromString(InString, Tangents);
}

FNumberFormattingOptions FEaseCurveTangents::DefaultNumberFormattingOptions()
{
	FNumberFormattingOptions NumberFormat;
	NumberFormat.MinimumIntegralDigits = 1;
	NumberFormat.MinimumFractionalDigits = 2;
	NumberFormat.MaximumFractionalDigits = 2;
	NumberFormat.UseGrouping = false;
	return NumberFormat;
}

FEaseCurveTangents::FEaseCurveTangents(const FString& InTangentsString)
{
	FEaseCurveTangents::FromString(InTangentsString, *this);
}

bool FEaseCurveTangents::IsNearlyEqual(const FEaseCurveTangents& InOther, const double InErrorTolerance) const
{
	return FMath::IsNearlyEqual(Start, InOther.Start, InErrorTolerance)
		&& FMath::IsNearlyEqual(StartWeight, InOther.StartWeight, InErrorTolerance)
		&& FMath::IsNearlyEqual(End, InOther.End, InErrorTolerance)
		&& FMath::IsNearlyEqual(EndWeight, InOther.EndWeight, InErrorTolerance);
}

FText FEaseCurveTangents::ToDisplayText() const
{
	const FNumberFormattingOptions NumberFormat = DefaultNumberFormattingOptions();
	const FText StartTangentText = FText::AsNumber(Start, &NumberFormat);
	const FText StartTangentWeightText = FText::AsNumber(StartWeight, &NumberFormat);
	const FText EndTangentText = FText::AsNumber(End, &NumberFormat);
	const FText EndTangentWeightText = FText::AsNumber(EndWeight, &NumberFormat);
	return FText::Format(LOCTEXT("TangentText", "{0}, {1} - {2}, {3}")
		, { StartTangentText, StartTangentWeightText, EndTangentText, EndTangentWeightText });
}

FString FEaseCurveTangents::ToDisplayString() const
{
	return ToDisplayText().ToString();
}

FString FEaseCurveTangents::ToJson() const
{
	// Convert two points/tangents to four cubic bezier points 
	TStaticArray<double, 4> CubicBezierPoints;
	ToCubicBezier(CubicBezierPoints);

	const FNumberFormattingOptions NumberFormat = DefaultNumberFormattingOptions();
	const FText PointAText = FText::AsNumber(CubicBezierPoints[0], &NumberFormat);
	const FText PointBText = FText::AsNumber(CubicBezierPoints[1], &NumberFormat);
	const FText PointCText = FText::AsNumber(CubicBezierPoints[2], &NumberFormat);
	const FText PointDText = FText::AsNumber(CubicBezierPoints[3], &NumberFormat);

	return FString::Printf(TEXT("%s, %s, %s, %s")
		, *PointAText.ToString(), *PointBText.ToString(), *PointCText.ToString(), *PointDText.ToString());
}

FText FEaseCurveTangents::GetStartTangentText() const
{
	const FNumberFormattingOptions NumberFormat = DefaultNumberFormattingOptions();
	const FText StartTangentText = FText::AsNumber(Start, &NumberFormat);
	const FText StartTangentWeightText = FText::AsNumber(StartWeight, &NumberFormat);
	return FText::Format(LOCTEXT("StartTangentText", "Start: {0}, {1}"), { StartTangentText, StartTangentWeightText });
}

FText FEaseCurveTangents::GetEndTangentText() const
{
	const FNumberFormattingOptions NumberFormat = DefaultNumberFormattingOptions();
	const FText EndTangentText = FText::AsNumber(End, &NumberFormat);
	const FText EndTangentWeightText = FText::AsNumber(EndWeight, &NumberFormat);
	return FText::Format(LOCTEXT("EndTangentText", "End: {0}, {1}"), { EndTangentText, EndTangentWeightText });
}

FText FEaseCurveTangents::GetCubicBezierText() const
{
	return FText::FromString(ToJson());
}

FEaseCurveTangents FEaseCurveTangents::MakeFromCubicBezier(const TStaticArray<double, 4>& InPoints)
{
	FEaseCurveTangents OutTangents;
	OutTangents.FromCubicBezier(InPoints);
	return MoveTemp(OutTangents);
}

bool FEaseCurveTangents::FromCubicBezier(const TStaticArray<double, 4>& InPoints)
{
	using namespace UE::EaseCurveTool;

	const FVector2d StartDir(InPoints[0], InPoints[1]);
	const FVector2d EndDir(1.0 - InPoints[2], 1.0 - InPoints[3]);

	Start = SEaseCurveEditor::CalcTangent(StartDir);
	End = SEaseCurveEditor::CalcTangent(EndDir);

	StartWeight = StartDir.Size();
	EndWeight = EndDir.Size();

	return true;
}

bool FEaseCurveTangents::ToCubicBezier(TStaticArray<double, 4>& OutPoints) const
{
	using namespace UE::EaseCurveTool;

	const FVector2d StartDir = SEaseCurveEditor::CalcTangentDir(Start) * StartWeight;
	const FVector2d EndDir = SEaseCurveEditor::CalcTangentDir(End) * EndWeight;

	OutPoints[0] = StartDir.X;
	OutPoints[1] = StartDir.Y;
	OutPoints[2] = 1.0 - EndDir.X;
	OutPoints[3] = 1.0 - EndDir.Y;

	return true;
}

void FEaseCurveTangents::Normalize(const FFrameNumber& InFrameNumber, const double InValue
	, const FFrameNumber& InNextFrameNumber, const double InNextValue
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution)
{
	using namespace UE::EaseCurveTool;

	// Convert frame time from tick resolution to display rate
	const FFrameTime FrameTimeDifference = InNextFrameNumber - InFrameNumber;
	const FQualifiedFrameTime QualifiedFrameTime(FrameTimeDifference, InTickResolution);
	const FFrameTime ConvertedFrameTime = QualifiedFrameTime.ConvertTo(InDisplayRate);

	// Create time/value range scale factor
	FVector2d TimeValueRange;
	TimeValueRange.X = InDisplayRate.AsSeconds(ConvertedFrameTime);
	TimeValueRange.Y = FMath::Abs(InNextValue - InValue);

	const double ScaleFactor = InTickResolution.AsDecimal();

	// Convert tangent angles to grid coordinates
	FVector2d StartDir = SEaseCurveEditor::CalcTangentDir(Start * ScaleFactor);
	FVector2d EndDir = SEaseCurveEditor::CalcTangentDir(End * ScaleFactor);

	StartDir *= StartWeight;
	EndDir *= EndWeight;

	// Scale down tangent grid coordinates to normalized range of InTimeValueRange
	auto SafeDivide = [](const double InNumerator, const double InDenominator)
		{
			return FMath::IsNearlyZero(InDenominator) ? InNumerator : (InNumerator / InDenominator);
		};
	StartDir.X = SafeDivide(StartDir.X, TimeValueRange.X);
	StartDir.Y = SafeDivide(StartDir.Y, TimeValueRange.Y);
	EndDir.X = SafeDivide(EndDir.X, TimeValueRange.X);
	EndDir.Y = SafeDivide(EndDir.Y, TimeValueRange.Y);

	// Set new weights and tangents
	StartWeight = StartDir.Size();
	EndWeight = EndDir.Size();

	StartDir.Normalize();
	EndDir.Normalize();

	Start = SEaseCurveEditor::CalcTangent(StartDir);
	End = SEaseCurveEditor::CalcTangent(EndDir);
}

void FEaseCurveTangents::ScaleUp(const FFrameNumber& InFrameNumber, const double InValue
	, const FFrameNumber& InNextFrameNumber, const double InNextValue
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution)
{
	using namespace UE::EaseCurveTool;

	// Convert frame time from tick resolution to display rate
	const FFrameTime FrameTimeDifference = InNextFrameNumber - InFrameNumber;
	const FQualifiedFrameTime QualifiedFrameTime(FrameTimeDifference, InTickResolution);
	const FFrameTime ConvertedFrameTime = QualifiedFrameTime.ConvertTo(InDisplayRate);

	// Create time/value range scale factor
	FVector2d TimeValueRange;
	TimeValueRange.X = InDisplayRate.AsSeconds(ConvertedFrameTime);
	TimeValueRange.Y = FMath::Abs(InNextValue - InValue);

	FVector2d StartDir = SEaseCurveEditor::CalcTangentDir(Start);
	FVector2d EndDir = SEaseCurveEditor::CalcTangentDir(End);

	StartDir *= StartWeight;
	EndDir *= EndWeight;

	StartDir *= TimeValueRange;
	EndDir *= TimeValueRange;

	// Set new weights
	StartWeight = StartDir.Size();
	EndWeight = EndDir.Size();

	// Normalize vector and set new tangents
	FVector2d NormalizedStartDir = StartDir;
	FVector2d NormalizedEndDir = EndDir;

	const double ScaleFactor = InTickResolution.AsDecimal();
	NormalizedStartDir.X *= ScaleFactor;
	NormalizedEndDir.X *= ScaleFactor;

	NormalizedStartDir.Normalize();
	NormalizedEndDir.Normalize();

	Start = SEaseCurveEditor::CalcTangent(NormalizedStartDir);
	End = SEaseCurveEditor::CalcTangent(NormalizedEndDir);
}

FEaseCurveTangents FEaseCurveTangents::Average(const TArray<FEaseCurveTangents>& InTangentArray)
{
	const int32 KeySetTangentsCount = InTangentArray.Num();
	if (KeySetTangentsCount == 0)
	{
		return FEaseCurveTangents();
	}

	double TotalStartTangent = 0.0;
	double TotalStartWeight = 0.0;
	double TotalEndTangent = 0.0;
	double TotalEndWeight = 0.0;

	for (const FEaseCurveTangents& Tangent : InTangentArray)
	{
		TotalStartTangent += Tangent.Start;
		TotalStartWeight += Tangent.StartWeight;
		TotalEndTangent += Tangent.End;
		TotalEndWeight += Tangent.EndWeight;
	}

	FEaseCurveTangents AverageTangents;
	AverageTangents.Start = TotalStartTangent / KeySetTangentsCount;
	AverageTangents.StartWeight = TotalStartWeight / KeySetTangentsCount;
	AverageTangents.End = TotalEndTangent / KeySetTangentsCount;
	AverageTangents.EndWeight = TotalEndWeight / KeySetTangentsCount;

	return AverageTangents;
}

double FEaseCurveTangents::CalculateCurveLength(const int32 SampleCount) const
{
	TStaticArray<double, 4> CubicBezierPoints;
	ToCubicBezier(CubicBezierPoints);

	const TArray<FVector2d> Points = { 
			FVector2d(0.0, 0.0), 
			FVector2d(CubicBezierPoints[0], CubicBezierPoints[1]), 
			FVector2d(CubicBezierPoints[2], CubicBezierPoints[3]), 
			FVector2d(1.0, 1.0)
		};

	auto EvaluatePoint = [](const double InTime, double InStart, double InControlA, double InControlB, double InEnd) -> double
		{
			return InStart * (1.0 - InTime) * (1.0 - InTime) * (1.0 - InTime)
				+ 3.0 * InControlA * (1.0 - InTime) * (1.0 - InTime) * InTime
				+ 3.0 * InControlB * (1.0 - InTime) * InTime * InTime
				+ InEnd * InTime * InTime * InTime;
		};

	// Sample points along the curve and use those points to calculate the length of the curve
	FVector2d SamplePoint;
	FVector2d PreviousSamplePoint = FVector2d::ZeroVector;
	double Length = 0.0;

	for (int32 Index = 0; Index <= SampleCount; Index++)
	{
		const double Time = (double)Index / (double)SampleCount;
				
		SamplePoint.X = EvaluatePoint(Time, Points[0].X, Points[1].X, Points[2].X, Points[3].X);
		SamplePoint.Y = EvaluatePoint(Time, Points[0].Y, Points[1].Y, Points[2].Y, Points[3].Y);

		if (Index > 0)
		{
			const double DifferenceX = SamplePoint.X - PreviousSamplePoint.X;
			const double DifferenceY = SamplePoint.Y - PreviousSamplePoint.Y;
			Length += FMath::Sqrt(DifferenceX * DifferenceX + DifferenceY * DifferenceY);
		}

		PreviousSamplePoint = SamplePoint;
	}

	return Length;
}

#undef LOCTEXT_NAMESPACE
