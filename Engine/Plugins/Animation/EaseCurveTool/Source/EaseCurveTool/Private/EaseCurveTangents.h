// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Math/UnrealMathUtility.h"
#include "EaseCurveTangents.generated.h"

class FString;
class FText;
struct FFrameRate;

/** Generic representation of a curve constructed of two tangents and their weights, one for each end of the curve. */
USTRUCT()
struct FEaseCurveTangents
{
	GENERATED_BODY()

public:
	static bool FromString(const FString& InString, FEaseCurveTangents& OutTangents);
	static bool CanParseString(const FString& InString);

	static FNumberFormattingOptions DefaultNumberFormattingOptions();

	static FEaseCurveTangents Average(const TArray<FEaseCurveTangents>& InTangentArray);

	FEaseCurveTangents() {}
	FEaseCurveTangents(const double InStart, const double InStartWeight, const double InEnd, const double InEndWeight)
		: Start(InStart), StartWeight(InStartWeight)
		, End(InEnd), EndWeight(InEndWeight)
	{}
	FEaseCurveTangents(const FRichCurveKey& InRichCurveKey)
		: Start(InRichCurveKey.LeaveTangent), StartWeight(InRichCurveKey.LeaveTangentWeight)
		, End(InRichCurveKey.ArriveTangent), EndWeight(InRichCurveKey.ArriveTangentWeight)
	{}
	FEaseCurveTangents(const FRichCurveKey& InStartRichCurveKey, const FRichCurveKey& InEndRichCurveKey)
		: Start(InStartRichCurveKey.LeaveTangent), StartWeight(InStartRichCurveKey.LeaveTangentWeight)
		, End(InEndRichCurveKey.ArriveTangent), EndWeight(InEndRichCurveKey.ArriveTangentWeight)
	{}
	FEaseCurveTangents(const FMovieSceneDoubleValue& InMovieSceneDoubleValue)
		: Start(InMovieSceneDoubleValue.Tangent.LeaveTangent), StartWeight(InMovieSceneDoubleValue.Tangent.LeaveTangentWeight)
		, End(InMovieSceneDoubleValue.Tangent.ArriveTangent), EndWeight(InMovieSceneDoubleValue.Tangent.ArriveTangentWeight)
	{}
	FEaseCurveTangents(const FMovieSceneDoubleValue& InStartMovieSceneDoubleValue, const FMovieSceneDoubleValue& InEndMovieSceneDoubleValue)
		: Start(InStartMovieSceneDoubleValue.Tangent.LeaveTangent), StartWeight(InStartMovieSceneDoubleValue.Tangent.LeaveTangentWeight)
		, End(InEndMovieSceneDoubleValue.Tangent.ArriveTangent), EndWeight(InEndMovieSceneDoubleValue.Tangent.ArriveTangentWeight)
	{}
	FEaseCurveTangents(const FMovieSceneFloatValue& InMovieSceneFloatValue)
		: Start(InMovieSceneFloatValue.Tangent.LeaveTangent), StartWeight(InMovieSceneFloatValue.Tangent.LeaveTangentWeight)
		, End(InMovieSceneFloatValue.Tangent.ArriveTangent), EndWeight(InMovieSceneFloatValue.Tangent.ArriveTangentWeight)
	{}
	FEaseCurveTangents(const FMovieSceneFloatValue& InStartMovieSceneFloatValue, const FMovieSceneFloatValue& InEndMovieSceneFloatValue)
		: Start(InStartMovieSceneFloatValue.Tangent.LeaveTangent), StartWeight(InStartMovieSceneFloatValue.Tangent.LeaveTangentWeight)
		, End(InEndMovieSceneFloatValue.Tangent.ArriveTangent), EndWeight(InEndMovieSceneFloatValue.Tangent.ArriveTangentWeight)
	{}
	/** Constructor from string consisting of cubic bezier points. Ex. "0.45, 0.34, 0.0, 1.00" */
	explicit FEaseCurveTangents(const FString& InTangentsString);

	FORCEINLINE bool operator==(const FEaseCurveTangents& InRhs) const
	{
		return Start == InRhs.Start && StartWeight == InRhs.StartWeight && End == InRhs.End && EndWeight == InRhs.EndWeight;
	}
	FORCEINLINE bool operator!=(const FEaseCurveTangents& InRhs) const
	{
		return !(*this == InRhs);
	}

	bool IsNearlyEqual(const FEaseCurveTangents& InOther, const double InErrorTolerance = UE_DOUBLE_SMALL_NUMBER) const;

	FText ToDisplayText() const;
	FString ToDisplayString() const;

	FString ToJson() const;

	FText GetStartTangentText() const;
	FText GetEndTangentText() const;
	FText GetCubicBezierText() const;

	static FEaseCurveTangents MakeFromCubicBezier(const TStaticArray<double, 4>& InPoints);

	bool FromCubicBezier(const TStaticArray<double, 4>& InPoints);
	bool ToCubicBezier(TStaticArray<double, 4>& OutPoints) const;

	void Normalize(const FFrameNumber& InFrameNumber, const double InValue
		, const FFrameNumber& InNextFrameNumber, const double InNextValue
		, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution);

	void ScaleUp(const FFrameNumber& InFrameNumber, const double InValue
		, const FFrameNumber& InNextFrameNumber, const double InNextValue
		, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution);

	/** Calculates the length of this curve. The higher the sample count, the higher the calculation precision. */
	double CalculateCurveLength(const int32 InSampleCount = 10) const;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurveTangents")
	double Start = 0.0;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurveTangents")
	double StartWeight = 0.0;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurveTangents")
	double End = 0.0;

	UPROPERTY(Config, EditAnywhere, Category = "EaseCurveTangents")
	double EndWeight = 0.0;
};
