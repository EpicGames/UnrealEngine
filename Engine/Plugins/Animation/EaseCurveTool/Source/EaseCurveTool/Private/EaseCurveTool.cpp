// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveTool.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "EaseCurveLibrary.h"
#include "EaseCurvePreset.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolSettings.h"
#include "EaseCurveToolTab.h"
#include "EaseCurveToolUtils.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Factories/CurveFactory.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISequencer.h"
#include "ISettingsModule.h"
#include "Math/UnrealMathUtility.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ScopedTransaction.h"
#include "MVVM/Selection/Selection.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SEaseCurveTool.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "EaseCurveTool"

namespace UE::EaseCurveTool
{

FEaseCurveTool::FEaseCurveTool(const TSharedRef<ISequencer>& InSequencer)
	: SequencerWeak(InSequencer)
	, ToolTab(MakeShared<FEaseCurveToolTab>(*this))
	, CommandList(MakeShared<FUICommandList>())
{
	EaseCurve = NewObject<UEaseCurve>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional);

	ToolSettings = GetMutableDefault<UEaseCurveToolSettings>();

	WeakPresetLibrary = ToolSettings->GetPresetLibrary();

	UpdateEaseCurveFromKeySelections();

	InSequencer->GetSelectionChangedObjectGuids().AddRaw(this, &FEaseCurveTool::OnSequencerSelectionChanged);
	InSequencer->OnMovieSceneDataChanged().AddRaw(this, &FEaseCurveTool::OnMovieSceneDataChanged);
	if (const TSharedPtr<FCurveEditor> CurveEditor = FEaseCurveToolUtils::GetCurveEditorFromSequencer(InSequencer))
	{
		CurveEditor->Selection.OnSelectionChanged().AddRaw(this, &FEaseCurveTool::OnCurveEditorSelectionChanged);
	}

	ToolTab->OnVisibilityChanged().AddRaw(this, &FEaseCurveTool::OnTabVisibilityChanged);

	const FEaseCurveToolCommands& ToolCommands = FEaseCurveToolCommands::Get();

	CommandList->MapAction(
		ToolCommands.ToggleToolTabVisible,
		FExecuteAction::CreateRaw(this, &FEaseCurveTool::ToggleToolTabVisible),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &FEaseCurveTool::IsToolTabVisible));

	ToolTab->Init();

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FEaseCurveTool::~FEaseCurveTool()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	if (const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		Sequencer->GetSelectionChangedObjectGuids().RemoveAll(this);
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
		
		if (const TSharedPtr<FCurveEditor> CurveEditor = FEaseCurveToolUtils::GetCurveEditorFromSequencer(Sequencer.ToSharedRef()))
		{
			CurveEditor->Selection.OnSelectionChanged().RemoveAll(this);
		}
	}

	ToolTab->OnVisibilityChanged().RemoveAll(this);
	ToolTab->Shutdown();
}

void FEaseCurveTool::OnSequencerSelectionChanged(TArray<FGuid> InObjectGuids)
{
	bRefreshForKeySelectionChange = true;
}

void FEaseCurveTool::OnMovieSceneDataChanged(const EMovieSceneDataChangeType InMetaData)
{
	bRefreshForKeySelectionChange = true;
}

void FEaseCurveTool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EaseCurve);
}

FString FEaseCurveTool::GetReferencerName() const
{
	return TEXT("EaseCurveTool");
}

TSharedPtr<ISequencer> FEaseCurveTool::GetSequencer() const
{
	return SequencerWeak.Pin();
}

bool FEaseCurveTool::IsToolTabVisible() const
{
	return ToolTab->IsToolTabVisible();
}

void FEaseCurveTool::ShowHideToolTab(const bool bInVisible)
{
	ToolTab->ShowHideToolTab(bInVisible);
}

void FEaseCurveTool::ToggleToolTabVisible()
{
	ShowHideToolTab(!IsToolTabVisible());
}

void FEaseCurveTool::OnTabVisibilityChanged(const bool bInVisible)
{
}

void FEaseCurveTool::OnCurveEditorSelectionChanged()
{
	// OnCurveEditorSelectionChanged is usually invoked as part of a transaction that changes selection, or as part of an undo / redo.
	// Generally, changes to selection in FCurveEditor should be wrapped by FScopedSelectionChangeEventSuppression, which effectively bundles multiple
	// changes into one broadcast at the end; FScopedSelectionChange constructs FScopedSelectionChangeEventSuppression. So, at this point, we needn't
	// worry about OnCurveEditorSelectionChanged being called multiple times in the same frame.
	if (
		// As UpdateEaseCurveFromKeySelections may in turn start a FScopedTransaction, we must process the change immediately so our changes become
		// part of the same transaction. 
		GUndo
		// If selection changes as part of an undo / redo, we must process immediately so any FScopedTransaction we start is not appended to the undo
		// buffer. If we didn't, then at end of tick we would create a new transaction, which would override all the undo stack past the current
		// transaction.
		|| GIsTransacting)
	{
		UpdateEaseCurveFromKeySelections();
	}
	else
	{
		// Otherwise we'll optimize a little by deferring the update in case more changes are made.
		bRefreshForKeySelectionChange = true;
	}
}

TSharedRef<SWidget> FEaseCurveTool::GenerateWidget()
{
	UpdateEaseCurveFromKeySelections();

	if (!ToolWidget.IsValid())
	{
		BindCommands();

		ToolWidget = SNew(SEaseCurveTool, SharedThis(this))
			.InitialTangents(GetEaseCurveTangents())
			.ToolOperation(this, &FEaseCurveTool::GetToolOperation);
	}

	return ToolWidget.ToSharedRef();
}

TSharedPtr<SEaseCurveTool> FEaseCurveTool::GetToolWidget() const
{
	return ToolWidget;
}

UEaseCurveLibrary* FEaseCurveTool::GetPresetLibrary() const
{
	return WeakPresetLibrary.Get();
}

void FEaseCurveTool::SetPresetLibrary(const TWeakObjectPtr<UEaseCurveLibrary>& InPresetLibrary)
{
	WeakPresetLibrary = InPresetLibrary;

	if (ToolSettings)
	{
		ToolSettings->SetPresetLibrary(InPresetLibrary.Get());
		ToolSettings->SaveConfig();
	}

	PresetLibraryChangedDelegate.Broadcast(WeakPresetLibrary);
}

TObjectPtr<UEaseCurve> FEaseCurveTool::GetToolCurve() const
{
	return EaseCurve;
}

FRichCurve* FEaseCurveTool::GetToolRichCurve() const
{
	return &EaseCurve->FloatCurve;
}

FEaseCurveTangents FEaseCurveTool::GetEaseCurveTangents() const
{
	return EaseCurve->GetTangents();
}

void FEaseCurveTool::SetEaseCurveTangents_Internal(const FEaseCurveTangents& InTangents, const EEaseCurveToolOperation InOperation, const bool bInBroadcastUpdate) const
{
	switch (InOperation)
	{
	case EEaseCurveToolOperation::InOut:
		EaseCurve->SetTangents(InTangents);
		break;
	case EEaseCurveToolOperation::In:
		EaseCurve->SetEndTangent(InTangents.End, InTangents.EndWeight);
		break;
	case EEaseCurveToolOperation::Out:
		EaseCurve->SetStartTangent(InTangents.Start, InTangents.StartWeight);
		break;
	}

	if (bInBroadcastUpdate)
	{
		EaseCurve->BroadcastUpdate();
	}
}

void FEaseCurveTool::SetEaseCurveTangents(const FEaseCurveTangents& InTangents
	, const EEaseCurveToolOperation InOperation
	, const bool bInBroadcastUpdate
	, const bool bInSetSequencerTangents
	, const FText& InTransactionText)
{
	if (InTangents == GetEaseCurveTangents())
	{
		return;
	}

	const FScopedTransaction Transaction(InTransactionText, !GIsTransacting);
	EaseCurve->Modify();

	SetEaseCurveTangents_Internal(InTangents, InOperation, bInBroadcastUpdate);

	if (bInSetSequencerTangents)
	{
		SetSequencerKeySelectionTangents(InTangents, InOperation);
	}

	KeyCache = FEaseCurveKeySelection(SequencerWeak.Pin());
}

void FEaseCurveTool::ResetEaseCurveTangents(const EEaseCurveToolOperation InOperation)
{
	FText TransactionText;

	switch (InOperation)
	{
	case EEaseCurveToolOperation::InOut:
		TransactionText = LOCTEXT("ResetTangents", "Reset Tangents");
		break;
	case EEaseCurveToolOperation::In:
		TransactionText = LOCTEXT("ResetEndTangents", "Reset End Tangents");
		break;
	case EEaseCurveToolOperation::Out:
		TransactionText = LOCTEXT("ResetStartTangents", "Reset Start Tangents");
		break;
	}

	const FScopedTransaction Transaction(TransactionText);
	EaseCurve->ModifyOwner();

	const FEaseCurveTangents ZeroTangents;
	SetEaseCurveTangents(ZeroTangents, InOperation, /*bInBroadcastUpdate=*/true, /*bInSetSequencerTangents=*/true, TransactionText);

	// Update ease curve combobox widget and zoom graph editor
	if (ToolWidget.IsValid())
	{
		ToolWidget->SetTangents(ZeroTangents, InOperation, /*bInSetEaseCurve=*/false, /*bInBroadcastUpdate=*/false, /*bInSetSequencerTangents=*/false);
	}
}

void FEaseCurveTool::FlattenOrStraightenTangents(const EEaseCurveToolOperation InOperation, const bool bInFlattenTangents)
{
	FText TransactionText;
	if (bInFlattenTangents)
	{
		switch (InOperation)
		{
		case EEaseCurveToolOperation::InOut:
			TransactionText = LOCTEXT("FlattenTangents", "Flatten Tangents");
			break;
		case EEaseCurveToolOperation::In:
			TransactionText = LOCTEXT("FlattenEndTangents", "Flatten End Tangents");
			break;
		case EEaseCurveToolOperation::Out:
			TransactionText = LOCTEXT("FlattenStartTangents", "Flatten Start Tangents");
			break;
		}
	}
	else
	{
		switch (InOperation)
		{
		case EEaseCurveToolOperation::InOut:
			TransactionText = LOCTEXT("StraightenTangents", "Straighten Tangents");
			break;
		case EEaseCurveToolOperation::In:
			TransactionText = LOCTEXT("StraightenEndTangents", "Straighten End Tangents");
			break;
		case EEaseCurveToolOperation::Out:
			TransactionText = LOCTEXT("StraightenStartTangents", "Straighten Start Tangents");
			break;
		}
	}
	const FScopedTransaction Transaction(TransactionText);
	EaseCurve->ModifyOwner();

	if (InOperation == EEaseCurveToolOperation::Out || InOperation == EEaseCurveToolOperation::InOut)
	{
		EaseCurve->FlattenOrStraightenTangents(EaseCurve->GetStartKeyHandle(), bInFlattenTangents);
	}
	if (InOperation == EEaseCurveToolOperation::In || InOperation == EEaseCurveToolOperation::InOut)
	{
		EaseCurve->FlattenOrStraightenTangents(EaseCurve->GetEndKeyHandle(), bInFlattenTangents);
	}

	const FEaseCurveTangents NewTangents = EaseCurve->GetTangents();
	SetEaseCurveTangents(NewTangents, InOperation, /*bInBroadcastUpdate=*/true, /*bInSetSequencerTangents=*/true, TransactionText);

	// Update ease curve combobox widget and zoom graph editor
	if (ToolWidget.IsValid())
	{
		ToolWidget->SetTangents(NewTangents, InOperation, /*bInSetEaseCurve=*/false, /*bInBroadcastUpdate=*/false, /*bInSetSequencerTangents=*/false);
	}
}

bool FEaseCurveTool::CanApplyQuickEaseToSequencerKeySelections() const
{
	FEaseCurveTangents Tangents;
	return FEaseCurveTangents::FromString(ToolSettings->GetQuickEaseTangents(), Tangents);
}

void FEaseCurveTool::ApplyQuickEaseToSequencerKeySelections(const EEaseCurveToolOperation InOperation)
{
	FEaseCurveTangents Tangents;
	if (!FEaseCurveTangents::FromString(ToolSettings->GetQuickEaseTangents(), Tangents))
	{
		UE_LOGF(LogTemp, Warning, "Ease curve tool failed to apply quick ease tangents: "
			"Could not parse configured quick ease tangent string.");
		return;
	}

	SetEaseCurveTangents(Tangents, InOperation, /*bInBroadcastUpdate=*/true, /*bInSetSequencerTangents=*/true);

	// Update ease curve combobox widget and zoom graph editor
	if (ToolWidget.IsValid())
	{
		ToolWidget->SetTangents(Tangents, InOperation, /*bInSetEaseCurve=*/false, /*bInBroadcastUpdate=*/false, /*bInSetSequencerTangents=*/false);
	}

	if (FEngineAnalytics::IsAvailable())
	{
		FString ParamValue;
		switch (InOperation)
		{
		case EEaseCurveToolOperation::InOut:
			ParamValue = TEXT("InOut");
			break;
		case EEaseCurveToolOperation::In:
			ParamValue = TEXT("In");
			break;
		case EEaseCurveToolOperation::Out:
			ParamValue = TEXT("Out");
			break;
		}
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.EaseCurveTool"), TEXT("QuickEase"), ParamValue);
	}
}

void FEaseCurveTool::SetSequencerKeySelectionTangents(const FEaseCurveTangents& InTangents, const EEaseCurveToolOperation InOperation)
{
	KeyCache = FEaseCurveKeySelection(SequencerWeak.Pin());

	if (KeyCache.GetTotalSelectedKeys() == 0)
	{
		return;
	}

	KeyCache.SetTangents(InTangents, InOperation, GetDisplayRate(), GetTickResolution(), ToolSettings->GetAutoFlipTangents());
}

void FEaseCurveTool::UpdateEaseCurveFromKeySelections()
{
	bRefreshForKeySelectionChange = false;

	KeyCache = FEaseCurveKeySelection(SequencerWeak.Pin());
	UpdateEaseCurveFromKeyCache();
}

FEaseCurveTangents FEaseCurveTool::GetAverageTangentsFromKeyCache()
{
	return KeyCache.AverageTangents(GetDisplayRate()
		, GetTickResolution(), ToolSettings->GetAutoFlipTangents());
}

void FEaseCurveTool::UpdateEaseCurveFromKeyCache()
{
	const FEaseCurveTangents AverageTangents = GetAverageTangentsFromKeyCache();

	SetEaseCurveTangents(AverageTangents, EEaseCurveToolOperation::InOut
		, /*bInBroadcastUpdate=*/true, /*bInSetSequencerTangents=*/false);

	// Update the preset combo box widget
	if (ToolWidget.IsValid())
	{
		ToolWidget->SetTangents(AverageTangents, EEaseCurveToolOperation::InOut
			, /*bInSetEaseCurve=*/false, /*bInBroadcastUpdate=*/false, /*bInSetSequencerTangents*/false);
	}
}

UCurveBase* FEaseCurveTool::CreateCurveAsset() const
{
	FAssetToolsModule* const AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>(TEXT("AssetTools"));
	if (!AssetToolsModule)
	{
		return nullptr;
	}

	FString OutNewPackageName;
	FString OutNewAssetName;
	AssetToolsModule->Get().CreateUniqueAssetName(TEXT("/Game/NewCurve"), TEXT(""), OutNewPackageName, OutNewAssetName);

	const TSharedRef<SDlgPickAssetPath> NewAssetDialog =
		SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("CreateExternalCurve", "Create External Curve"))
		.DefaultAssetPath(FText::FromString(OutNewPackageName));

	if (NewAssetDialog->ShowModal() != EAppReturnType::Cancel)
	{
		const FString PackageName = NewAssetDialog->GetFullAssetPath().ToString();
		const FName AssetName = FName(*NewAssetDialog->GetAssetName().ToString());

		UPackage* const Package = CreatePackage(*PackageName);
		
		// Create curve object
		UObject* NewCurveObject = nullptr;
		
		if (UCurveFactory* const CurveFactory = Cast<UCurveFactory>(NewObject<UFactory>(GetTransientPackage(), UCurveFactory::StaticClass())))
		{
			CurveFactory->CurveClass = UCurveFloat::StaticClass();
			NewCurveObject = CurveFactory->FactoryCreateNew(CurveFactory->GetSupportedClass(), Package, AssetName, RF_Public | RF_Standalone, nullptr, GWarn);
		}

		if (NewCurveObject)
		{
			UCurveBase* AssetCurve = nullptr;
			
			// Copy curve data from current curve to newly create curve
			UCurveFloat* const DestCurve = CastChecked<UCurveFloat>(NewCurveObject);
			if (EaseCurve && DestCurve)
			{
				DestCurve->bIsEventCurve = false;

				AssetCurve = DestCurve;

				for (auto It(EaseCurve->FloatCurve.GetKeyIterator()); It; ++It)
				{
					const FRichCurveKey& Key = *It;
					const FKeyHandle KeyHandle = DestCurve->FloatCurve.AddKey(Key.Time, Key.Value);
					DestCurve->FloatCurve.GetKey(KeyHandle) = Key;
				}
			}

			FAssetRegistryModule::AssetCreated(NewCurveObject);

			Package->GetOutermost()->MarkPackageDirty();

			return AssetCurve;
		}
	}

	return nullptr;
}

EEaseCurveToolOperation FEaseCurveTool::GetToolOperation() const
{
	return OperationMode;
}

void FEaseCurveTool::SetToolOperation(const EEaseCurveToolOperation InNewOperation)
{
	OperationMode = InNewOperation;
}

bool FEaseCurveTool::IsToolOperation(const EEaseCurveToolOperation InNewOperation) const
{
	return OperationMode == InNewOperation;
}

bool FEaseCurveTool::CanCopyTangentsToClipboard() const
{
	return true;
}

void FEaseCurveTool::CopyTangentsToClipboard() const
{
	FPlatformApplicationMisc::ClipboardCopy(*EaseCurve->GetTangents().ToJson());

	ShowNotificationMessage(LOCTEXT("EaseCurveToolTangentsCopied", "Ease Curve Tool Tangents Copied!"));
}

bool FEaseCurveTool::CanPasteTangentsFromClipboard() const
{
	FEaseCurveTangents Tangents;
	return TangentsFromClipboardPaste(Tangents);
}

void FEaseCurveTool::PasteTangentsFromClipboard() const
{
	FEaseCurveTangents Tangents;
	if (TangentsFromClipboardPaste(Tangents))
	{
		EaseCurve->SetTangents(Tangents);
	}
}

bool FEaseCurveTool::TangentsFromClipboardPaste(FEaseCurveTangents& OutTangents)
{
	FString ClipboardString;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardString);

	// Expects four comma separated cubic bezier points that define the curve
	return FEaseCurveTangents::FromString(ClipboardString, OutTangents);
}

bool FEaseCurveTool::IsKeyInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode) const
{
	const FKeyHandle StartKeyHandle = EaseCurve->GetStartKeyHandle();
	return (EaseCurve->FloatCurve.GetKeyInterpMode(StartKeyHandle) == InInterpMode
		&& EaseCurve->FloatCurve.GetKeyTangentMode(StartKeyHandle) == InTangentMode);
}

void FEaseCurveTool::SetKeyInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode) const
{
	const FKeyHandle StartKeyHandle = EaseCurve->GetStartKeyHandle();

	const FScopedTransaction Transaction(LOCTEXT("CurveEditor_SetInterpolationMode", "Select Interpolation Mode"));
	EaseCurve->ModifyOwner();

	EaseCurve->FloatCurve.SetKeyInterpMode(StartKeyHandle, InInterpMode);
	EaseCurve->FloatCurve.SetKeyTangentMode(StartKeyHandle, InTangentMode);

	if (InInterpMode != ERichCurveInterpMode::RCIM_Cubic)
	{
		FRichCurveKey& StartKey = EaseCurve->GetStartKey();
		StartKey.LeaveTangentWeight = 0.f;
		
		FRichCurveKey& EndKey = EaseCurve->GetEndKey();
		EndKey.ArriveTangentWeight = 0.f;
	}

	TArray<FRichCurveEditInfo> ChangedCurveEditInfos;
	ChangedCurveEditInfos.Add(FRichCurveEditInfo(&EaseCurve->FloatCurve));
	EaseCurve->OnCurveChanged(ChangedCurveEditInfos);
}

void FEaseCurveTool::BeginTransaction(const FText& InDescription) const
{
	if (GEditor)
	{
		EaseCurve->ModifyOwnerChange();

		GEditor->BeginTransaction(InDescription);
	}
}

void FEaseCurveTool::EndTransaction() const
{
	if (GEditor)
	{
		GEditor->EndTransaction();
	}
}

void FEaseCurveTool::PostUndo(const bool bInSuccess)
{
	UpdateEaseCurveFromKeySelections();
}

void FEaseCurveTool::PostRedo(const bool bInSuccess)
{
	UpdateEaseCurveFromKeySelections();
}

bool FEaseCurveTool::IsTickable() const
{
	return true;
}

ETickableTickType FEaseCurveTool::GetTickableTickType() const
{
	return ETickableTickType::Always;
}

TStatId FEaseCurveTool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FEaseCurveTool, STATGROUP_Tickables);
}

void FEaseCurveTool::Tick(float InDeltaTime)
{
	using namespace UE::EaseCurveTool;

	if (bRefreshForKeySelectionChange)
	{
		UpdateEaseCurveFromKeySelections();
	}
}

void FEaseCurveTool::OpenToolSettings() const
{
	if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->ShowViewer(ToolSettings->GetContainerName(), ToolSettings->GetCategoryName(), ToolSettings->GetSectionName());
	}
}

FFrameRate FEaseCurveTool::GetTickResolution() const
{
	if (const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		return Sequencer->GetFocusedTickResolution();
	}

	return FFrameRate();
}

FFrameRate FEaseCurveTool::GetDisplayRate() const
{
	if (const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		return Sequencer->GetFocusedDisplayRate();
	}
	return FFrameRate();
}

void FEaseCurveTool::ShowNotificationMessage(const FText& InMessageText)
{
	FNotificationInfo Info(InMessageText);
	Info.ExpireDuration = 3.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

void FEaseCurveTool::RecordPresetAnalytics(const TSharedPtr<FEaseCurvePreset>& InPreset, const FString& InLocation)
{
	if (!FEngineAnalytics::IsAvailable() || !InPreset.IsValid())
	{
		return;
	}

	// Only send analytics for default presets
	const TArray<FEaseCurvePreset> DefaultPresets = UEaseCurveLibrary::GetDefaultPresets();

	if (DefaultPresets.Contains(*InPreset))
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("Category"), InPreset->Category.ToString());
		Attributes.Emplace(TEXT("Name"), InPreset->Name.ToString());
		Attributes.Emplace(TEXT("Location"), InLocation);

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.EaseCurveTool.SetTangentsPreset"), Attributes);
	}
}

bool FEaseCurveTool::HasCachedKeysToEase() const
{
	return KeyCache.HasCachedKeysToEase();
}

EEaseCurveToolError FEaseCurveTool::GetSelectionError() const
{
	return KeyCache.GetSelectionError();
}

FText FEaseCurveTool::GetSelectionErrorText() const
{
	switch (GetSelectionError())
	{
	case EEaseCurveToolError::None:
		return FText::GetEmpty();
	case EEaseCurveToolError::LastKey:
		return LOCTEXT("LastSelectedKey", "No next key to ease to!");
	case EEaseCurveToolError::SameValues:
		return LOCTEXT("EqualValueKeys", "No different key values!");
	case EEaseCurveToolError::NoWeightedBrokenCubicTangents:
		return LOCTEXT("NoEaseCurve", "No weighted, broken, cubic tangents!");
	}
	return FText::GetEmpty();
}

bool FEaseCurveTool::IsCurveEditorSelection() const
{
	return KeyCache.IsCurveEditorSelection();
}

void FEaseCurveTool::ApplyTangents()
{
	SetEaseCurveTangents(GetEaseCurveTangents(), GetToolOperation(), /*bInBroadcastUpdate=*/true, /*bInSetSequencerTangents=*/true);
}

void FEaseCurveTool::ZoomToFit() const
{
	if (ToolWidget.IsValid())
	{
		ToolWidget->ZoomToFit();
	}
}

TSharedPtr<FUICommandList> FEaseCurveTool::GetCommandList() const
{
	return CommandList;
}

void FEaseCurveTool::BindCommands()
{
	const FEaseCurveToolCommands& EaseCurveToolCommands = FEaseCurveToolCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(EaseCurveToolCommands.OpenToolSettings
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::OpenToolSettings));

	CommandList->MapAction(EaseCurveToolCommands.Refresh
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::UpdateEaseCurveFromKeySelections));
	
	CommandList->MapAction(EaseCurveToolCommands.Apply
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::ApplyTangents));

	CommandList->MapAction(EaseCurveToolCommands.ZoomToFit
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::ZoomToFit));

	CommandList->MapAction(EaseCurveToolCommands.ToggleGridSnap
		, FExecuteAction::CreateUObject(ToolSettings, &UEaseCurveToolSettings::ToggleGridSnap)
		, FCanExecuteAction()
		, FIsActionChecked::CreateUObject(ToolSettings, &UEaseCurveToolSettings::GetGridSnap));

	CommandList->MapAction(EaseCurveToolCommands.ToggleAutoFlipTangents
		, FExecuteAction::CreateUObject(ToolSettings, &UEaseCurveToolSettings::ToggleAutoFlipTangents)
		, FCanExecuteAction()
		, FIsActionChecked::CreateUObject(ToolSettings, &UEaseCurveToolSettings::GetAutoFlipTangents));

	CommandList->MapAction(EaseCurveToolCommands.ToggleAutoZoomToFit
		, FExecuteAction::CreateUObject(ToolSettings, &UEaseCurveToolSettings::ToggleAutoZoomToFit)
		, FCanExecuteAction()
		, FIsActionChecked::CreateUObject(ToolSettings, &UEaseCurveToolSettings::GetAutoZoomToFit));

	CommandList->MapAction(EaseCurveToolCommands.SetOperationToEaseOut
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::SetToolOperation, EEaseCurveToolOperation::Out)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsToolOperation, EEaseCurveToolOperation::Out));

	CommandList->MapAction(EaseCurveToolCommands.SetOperationToEaseInOut
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::SetToolOperation, EEaseCurveToolOperation::InOut)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsToolOperation, EEaseCurveToolOperation::InOut));

	CommandList->MapAction(EaseCurveToolCommands.SetOperationToEaseIn
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::SetToolOperation, EEaseCurveToolOperation::In)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsToolOperation, EEaseCurveToolOperation::In));

	CommandList->MapAction(EaseCurveToolCommands.ResetTangents
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::ResetEaseCurveTangents, EEaseCurveToolOperation::InOut));

	CommandList->MapAction(EaseCurveToolCommands.ResetStartTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::ResetEaseCurveTangents, EEaseCurveToolOperation::Out));

	CommandList->MapAction(EaseCurveToolCommands.ResetEndTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::ResetEaseCurveTangents, EEaseCurveToolOperation::In));

	CommandList->MapAction(EaseCurveToolCommands.FlattenTangents
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::InOut, true));

	CommandList->MapAction(EaseCurveToolCommands.FlattenStartTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::Out, true));

	CommandList->MapAction(EaseCurveToolCommands.FlattenEndTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::In, true));

	CommandList->MapAction(EaseCurveToolCommands.StraightenTangents
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::InOut, false));

	CommandList->MapAction(EaseCurveToolCommands.StraightenStartTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::Out, false));

	CommandList->MapAction(EaseCurveToolCommands.StraightenEndTangent
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::FlattenOrStraightenTangents, EEaseCurveToolOperation::In, false));

	CommandList->MapAction(EaseCurveToolCommands.CopyTangents
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::CopyTangentsToClipboard)
		, FCanExecuteAction::CreateSP(this, &FEaseCurveTool::CanCopyTangentsToClipboard));

	CommandList->MapAction(EaseCurveToolCommands.PasteTangents
		, FExecuteAction::CreateSP(this, &FEaseCurveTool::PasteTangentsFromClipboard)
		, FCanExecuteAction::CreateSP(this, &FEaseCurveTool::CanPasteTangentsFromClipboard));

	CommandList->MapAction(EaseCurveToolCommands.CreateExternalCurveAsset
		, FExecuteAction::CreateSPLambda(this, [this]()
			{
				CreateCurveAsset();
			})
		, FCanExecuteAction());

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpConstant,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Constant, RCTM_Auto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Constant, RCTM_Auto));

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpLinear,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Linear, RCTM_Auto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Linear, RCTM_Auto));

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpCubicAuto,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Cubic, RCTM_Auto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Cubic, RCTM_Auto));

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpCubicSmartAuto,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Cubic, RCTM_SmartAuto),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Cubic, RCTM_SmartAuto));

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpCubicUser,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Cubic, RCTM_User),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Cubic, RCTM_User));

	CommandList->MapAction(EaseCurveToolCommands.SetKeyInterpCubicBreak,
		FExecuteAction::CreateSP(this, &FEaseCurveTool::SetKeyInterpMode, RCIM_Cubic, RCTM_Break),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FEaseCurveTool::IsKeyInterpMode, RCIM_Cubic, RCTM_Break));
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
