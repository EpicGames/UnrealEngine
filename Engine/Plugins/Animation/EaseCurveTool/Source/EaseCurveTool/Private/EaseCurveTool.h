// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorSelection.h"
#include "Curves/RichCurve.h"
#include "EaseCurve.h"
#include "EaseCurveKeySelection.h"
#include "EditorUndoClient.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SEaseCurvePreset.h"

class FUICommandList;
class ISequencer;
class UEaseCurve;
class UEaseCurveToolSettings;
enum class EMovieSceneDataChangeType;
struct FEaseCurvePreset;
struct FEaseCurveTangents;
struct FGuid;

namespace UE::EaseCurveTool
{

class FEaseCurveToolTab;
class SEaseCurveTool;

/** Current default and only implemented is DualKeyEdit. */
enum class EEaseCurveToolMode : uint8
{
	/** Edits the selected key's leave tangent and the next key's arrive tangent in the curve editor graph. */
	DualKeyEdit,
	/** Edits only the selected key.
	 * The leave tangent in the curve editor graph will set the sequence key arrive tangent.
	 * The arrive tangent in the curve editor graph will set the sequence key leave tangent. */
	SingleKeyEdit
};

enum class EEaseCurveToolOperation : uint8
{
	InOut,
	In,
	Out
};

DECLARE_DELEGATE_RetVal(UEaseCurveLibrary*, FOnGetEaseCurveSelectedLibrary)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnEaseCurveLibraryChanged, TWeakObjectPtr<UEaseCurveLibrary> /*InLibrary*/)

class FEaseCurveTool
	: public TSharedFromThis<FEaseCurveTool>
	, public FGCObject
	, public FSelfRegisteringEditorUndoClient
	, public FTickableEditorObject
{
public:
	static void ShowNotificationMessage(const FText& InMessageText);

	static void RecordPresetAnalytics(const TSharedPtr<FEaseCurvePreset>& InPreset, const FString& InLocation);

	/** Returns true if the clipboard paste data contains tangent information. */
	static bool TangentsFromClipboardPaste(FEaseCurveTangents& OutTangents);

	FEaseCurveTool(const TSharedRef<ISequencer>& InSequencer);

	virtual ~FEaseCurveTool() override;

	TSharedPtr<ISequencer> GetSequencer() const;

	bool IsToolTabVisible() const;
	void ShowHideToolTab(const bool bInVisible);
	void ToggleToolTabVisible();

	TSharedRef<SWidget> GenerateWidget();
	TSharedPtr<SEaseCurveTool> GetToolWidget() const;

	UEaseCurveLibrary* GetPresetLibrary() const;
	void SetPresetLibrary(const TWeakObjectPtr<UEaseCurveLibrary>& InPresetLibrary);

	/** Event that broadcasts when the underlying ease curve preset library has been changed */
	FOnEaseCurveLibraryChanged& OnPresetLibraryChanged() { return PresetLibraryChangedDelegate; }

	TObjectPtr<UEaseCurve> GetToolCurve() const;
	FRichCurve* GetToolRichCurve() const;

	FEaseCurveTangents GetEaseCurveTangents() const;

	/**
	 * Sets the internal ease curve tangents and optionally broadcasts a change event for the curve object.
	 * This is different from SetEaseCurveTangents_Internal in that it performs undo/redo transactions and
	 * optionally sets the selected tangents in the actual sequence.
	 */
	void SetEaseCurveTangents(const FEaseCurveTangents& InTangents
		, const EEaseCurveToolOperation InOperation
		, const bool bInBroadcastUpdate
		, const bool bInSetSequencerTangents
		, const FText& InTransactionText = NSLOCTEXT("EaseCurveTool", "SetEaseCurveTangents", "Set Ease Curve Tangents"));

	void ResetEaseCurveTangents(const EEaseCurveToolOperation InOperation);

	void FlattenOrStraightenTangents(const EEaseCurveToolOperation InOperation, const bool bInFlattenTangents);

	/** Creates a new external float curve from the internet curve editor curve. */
	UCurveBase* CreateCurveAsset() const;

	void SetSequencerKeySelectionTangents(const FEaseCurveTangents& InTangents, const EEaseCurveToolOperation InOperation = EEaseCurveToolOperation::InOut);

	bool CanApplyQuickEaseToSequencerKeySelections() const;
	void ApplyQuickEaseToSequencerKeySelections(const EEaseCurveToolOperation InOperation = EEaseCurveToolOperation::InOut);

	/** Updates the ease curve graph view based on the active sequencer or curve editor key selection. */
	void UpdateEaseCurveFromKeySelections();

	EEaseCurveToolOperation GetToolOperation() const;
	void SetToolOperation(const EEaseCurveToolOperation InNewOperation);
	bool IsToolOperation(const EEaseCurveToolOperation InNewOperation) const;

	bool CanCopyTangentsToClipboard() const;
	void CopyTangentsToClipboard() const;
	bool CanPasteTangentsFromClipboard() const;
	void PasteTangentsFromClipboard() const;

	bool IsKeyInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode) const;
	void SetKeyInterpMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode) const;

	void BeginTransaction(const FText& InDescription) const;
	void EndTransaction() const;

	void OpenToolSettings() const;

	FFrameRate GetTickResolution() const;
	FFrameRate GetDisplayRate() const;

	bool HasCachedKeysToEase() const;

	EEaseCurveToolError GetSelectionError() const;
	FText GetSelectionErrorText() const;

	/** @return True if the tool is operating on a curve editor key selection, False if operating on a Sequencer key selection */
	bool IsCurveEditorSelection() const;

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

	//~ Begin FEditorUndoClient
	virtual void PostUndo(const bool bInSuccess) override;
	virtual void PostRedo(const bool bInSuccess) override;
	//~ End FEditorUndoClient

	//~ Begin FTickableEditorObject
	virtual bool IsTickable() const override;
	virtual ETickableTickType GetTickableTickType() const override;
	virtual TStatId GetStatId() const override;
	virtual void Tick(float InDeltaTime) override;
	//~ End FTickableEditorObject

	TSharedPtr<FUICommandList> GetCommandList() const;

	FEaseCurveKeySelection& GetSelectedKeyCache() { return KeyCache; }

	FEaseCurveTangents GetAverageTangentsFromKeyCache();

protected:
	void ApplyTangents();
	void ZoomToFit() const;

	void BindCommands();

	/** 
	 * Sets the internal ease curve tangents and optionally broadcasts a change event for the curve object.
	 * Changing the internal ease curve tangents will be directly reflected in the ease curve editor graph.
	 */
	void SetEaseCurveTangents_Internal(const FEaseCurveTangents& InTangents, const EEaseCurveToolOperation InOperation, const bool bInBroadcastUpdate) const;

	void UpdateEaseCurveFromKeyCache();

	TWeakPtr<ISequencer> SequencerWeak;

	TSharedRef<FEaseCurveToolTab> ToolTab;

	TSharedPtr<FUICommandList> CommandList;

	TWeakObjectPtr<UEaseCurveLibrary> WeakPresetLibrary;
	FOnEaseCurveLibraryChanged PresetLibraryChangedDelegate;

	TObjectPtr<UEaseCurve> EaseCurve;

	TObjectPtr<UEaseCurveToolSettings> ToolSettings;

	EEaseCurveToolMode ToolMode = EEaseCurveToolMode::DualKeyEdit;
	EEaseCurveToolOperation OperationMode = EEaseCurveToolOperation::InOut;

	TSharedPtr<SEaseCurveTool> ToolWidget;

	/** Cached data set when a new sequencer selection is made. */
	FEaseCurveKeySelection KeyCache;

	bool bRefreshForKeySelectionChange = false;

	FCurveEditorSelection LastCurveEditorSelection;

private:
	void OnSequencerSelectionChanged(TArray<FGuid> InObjectGuids);
	void OnMovieSceneDataChanged(const EMovieSceneDataChangeType InMetaData);

	void OnTabVisibilityChanged(const bool bInVisible);
	
	void OnCurveEditorSelectionChanged();
};

} // namespace UE::EaseCurveTool
