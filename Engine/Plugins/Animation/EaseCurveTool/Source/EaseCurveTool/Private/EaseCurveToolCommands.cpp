// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolCommands.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolCommands"

void FEaseCurveToolCommands::RegisterCommands()
{
	const TSharedRef<FBindingContext> SharedThis = AsShared();

	UI_COMMAND(ToggleToolTabVisible
		, "Ease Curve Tool"
		, "Toggle the visibility of the Ease Curve Tool tab window"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(SharedThis
		, OpenToolSettings
		, TEXT("OpenToolSettings")
		, LOCTEXT("OpenToolSettings_Label", "Open Tool Settings...")
		, LOCTEXT("OpenToolSettings_ToolTip", "Open the editor settings for the the curve ease tool.")
		, FSlateIcon()
		, EUserInterfaceActionType::Button
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(SharedThis
		, Refresh
		, TEXT("Refresh")
		, LOCTEXT("Refresh_Label", "Refresh")
		, LOCTEXT("Refresh_ToolTip", "Refresh the ease curve editor graph tangents from the selected sequencer key(s).")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Refresh"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F5));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, Apply
		, TEXT("Apply")
		, LOCTEXT("Apply_Label", "Apply")
		, LOCTEXT("Apply_ToolTip", "Applies the ease curve to the selected sequencer key(s).")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("AssetEditor.Apply"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ZoomToFit
		, TEXT("ZoomToFit")
		, LOCTEXT("ZoomToFit_Label", "Zoom to Fit")
		, LOCTEXT("ZoomToFit_ToolTip", "Zoom the curve graph view to fit tangent handles.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("CurveEditorTools.SetFocusPlaybackRange"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, SetOperationToEaseOut
		, TEXT("SetOperationToEaseOut")
		, LOCTEXT("SetOperationToEaseOut_Label", "Ease Out")
		, LOCTEXT("SetOperationToEaseOut_ToolTip", "Set the operation of the Curve Ease Tool to Ease Out only. This will only set the Out (Leave) tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.ChevronLeft"))
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::O));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, SetOperationToEaseInOut
		, TEXT("SetOperationToEaseInOut")
		, LOCTEXT("SetOperationToEaseInOut_Label", "Ease In Out")
		, LOCTEXT("SetOperationToEaseInOut_ToolTip", "Set the operation of the Curve Ease Tool to both Ease Out and Ease In. This will set the Out (Leave) and In (Arrive) tangents.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.BulletPoint"))
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::U));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, SetOperationToEaseIn
		, TEXT("SetOperationToEaseIn")
		, LOCTEXT("SetOperationToEaseIn_Label", "Ease In")
		, LOCTEXT("SetOperationToEaseIn_ToolTip", "Set the operation of the Curve Ease Tool to Ease In only. This will only set the In (Arrive) tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.ChevronRight"))
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::I));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ToggleGridSnap
		, TEXT("ToggleGridSnap")
		, LOCTEXT("ToggleGridSnap_Label", "Snap To Grid")
		, LOCTEXT("ToggleGridSnap_ToolTip", "Toggle snapping of tangents to grid.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Snap"))
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::G));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ToggleAutoFlipTangents
		, TEXT("ToggleAutoFlipTangents")
		, LOCTEXT("ToggleAutoFlipTangents_Label", "Auto Flip Tangents")
		, LOCTEXT("ToggleAutoFlipTangents_ToolTip", "Toggle flipping of tangents when sequential key frame curve values are descending.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.InterpolationCubicAuto.Small"))
		, EUserInterfaceActionType::Check
		, FInputChord());

	UI_COMMAND(ToggleAutoZoomToFit
		, "Auto Zoom To Fit"
		, "Auto zoom the graph editor to fit the tangent handles after they have been changed."
		, EUserInterfaceActionType::Check
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ResetTangents
		, TEXT("ResetBothTangents")
		, LOCTEXT("ResetBothTangents_Label", "Both Tangents")
		, LOCTEXT("ResetBothTangents_ToolTip", "Set the In / Out tangent values and weights to 0.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::R));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ResetStartTangent
		, TEXT("ResetStartTangent")
		, LOCTEXT("ResetStartTangent_Label", "Start Tangent")
		, LOCTEXT("ResetStartTangent_ToolTip", "Set the start tangent value and weight to 0.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::R, EModifierKey::Control));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, ResetEndTangent
		, TEXT("ResetEndTangent")
		, LOCTEXT("ResetEndTangent_Label", "End Tangent")
		, LOCTEXT("ResetEndTangent_ToolTip", "Set the end tangent value and weight to 0.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::R, EModifierKey::Alt));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, FlattenTangents
		, TEXT("FlattenBothTangents")
		, LOCTEXT("FlattenBothTangents_Label", "Both Tangents")
		, LOCTEXT("FlattenBothTangents_ToolTip", "Flatten the start and end tangents.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.FlattenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::T));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, FlattenStartTangent
		, TEXT("FlattenStartTangent")
		, LOCTEXT("FlattenStartTangent_Label", "Start Tangent")
		, LOCTEXT("FlattenStartTangent_ToolTip", "Flatten the start tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.FlattenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::T, EModifierKey::Control));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, FlattenEndTangent
		, TEXT("FlattenEndTangent")
		, LOCTEXT("FlattenEndTangent_Label", "End Tangent")
		, LOCTEXT("FlattenEndTangent_ToolTip", "Flatten the end tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.FlattenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::T, EModifierKey::Alt));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, StraightenTangents
		, TEXT("StraightenBothTangents")
		, LOCTEXT("StraightenBothTangents_Label", "Both Tangents")
		, LOCTEXT("StraightenBothTangents_ToolTip", "Straighten the start and end tangents.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.StraightenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::S));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, StraightenStartTangent
		, TEXT("StraightenStartTangent")
		, LOCTEXT("StraightenStartTangent_Label", "Start Tangent")
		, LOCTEXT("StraightenStartTangent_ToolTip", "Straighten the start tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.StraightenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::S, EModifierKey::Control));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, StraightenEndTangent
		, TEXT("StraightenEndTangent")
		, LOCTEXT("StraightenEndTangent_Label", "End Tangent")
		, LOCTEXT("StraightenEndTangent_ToolTip", "Straighten the end tangent.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.StraightenTangents"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::S, EModifierKey::Alt));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, CopyTangents
		, TEXT("CopyTangents")
		, LOCTEXT("CopyTangents_Label", "Copy Tangents")
		, LOCTEXT("CopyTangents_ToolTip", "Copy the set of In / Out tangent values and weights.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCommands.Copy"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::C, EModifierKey::Control));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, PasteTangents
		, TEXT("PasteTangents")
		, LOCTEXT("PasteTangents_Label", "Paste Tangents")
		, LOCTEXT("PasteTangents_ToolTip", "Paste the set of In / Out tangent values and weights.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCommands.Paste"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::V, EModifierKey::Control));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, CreateExternalCurveAsset
		, TEXT("CreateExternalCurve")
		, LOCTEXT("CreateExternalCurve_Label", "Create External Curve...")
		, LOCTEXT("CreateExternalCurve_ToolTip", "Create an external curve asset using this internal curve.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "TimelineEditor.AddCurveAssetTrack")
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SetKeyInterpConstant, "Constant", "Constant interpolation", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Five));
	UI_COMMAND(SetKeyInterpLinear, "Linear", "Linear interpolation", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Four));
	UI_COMMAND(SetKeyInterpCubicSmartAuto, "Smart Auto", "Cubic interpolation - Smart Automatic tangents", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Zero));
	UI_COMMAND(SetKeyInterpCubicAuto, "Auto", "Cubic interpolation - Automatic tangents", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::One));
	UI_COMMAND(SetKeyInterpCubicUser, "User", "Cubic interpolation - User flat tangents", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Two));
	UI_COMMAND(SetKeyInterpCubicBreak, "Break", "Cubic interpolation - User broken tangents", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Three));
	UI_COMMAND(SetKeyInterpToggleWeighted, "Weighted Tangents", "Toggle weighted tangents for cubic interpolation modes. Only supported on some curve types", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::W));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, QuickEase
		, TEXT("QuickEase")
		, LOCTEXT("QuickEase_Label", "Quick Ease")
		, LOCTEXT("QuickEase_ToolTip", "Apply the quick ease preset. This will be disabled if there is no valid quick ease preset setup.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.BulletPoint"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F8));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, QuickEaseIn
		, TEXT("QuickEaseIn")
		, LOCTEXT("QuickEaseIn_Label", "Quick Ease In")
		, LOCTEXT("QuickEaseIn_ToolTip", "Apply the quick ease in preset. This will be disabled if there is no valid quick ease preset setup.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.ChevronRight"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F8, EModifierKey::Shift));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, QuickEaseOut
		, TEXT("QuickEaseOut")
		, LOCTEXT("QuickEaseOut_Label", "Quick Ease Out")
		, LOCTEXT("QuickEaseOut_ToolTip", "Apply the quick ease out preset. This will be disabled if there is no valid quick ease preset setup.")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.ChevronLeft"))
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F8, EModifierKey::Command | EModifierKey::Shift));

	FUICommandInfo::MakeCommandInfo(SharedThis
		, SaveCurveEditorSelection
		, TEXT("SaveCurveEditorSelection")
		, LOCTEXT("SaveCurveEditorSelection_Label", "Save Curve Editor Selection")
		, LOCTEXT("SaveCurveEditorSelection_ToolTip", "Saves the current curve editor key selection as a new ease curve preset\n\n*NOTE* The selected key must be using broken and weighted tangents")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.ChevronLeft"))
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
