// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FEaseCurveToolCommands : public TCommands<FEaseCurveToolCommands>
{
public:
	FEaseCurveToolCommands()
		: TCommands<FEaseCurveToolCommands>(TEXT("EaseCurveTool")
			, NSLOCTEXT("EaseCurveToolCommands", "EaseCurveToolCommands", "Ease Curve Tool Commands")
			, NAME_None
			, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ToggleToolTabVisible;

	TSharedPtr<FUICommandInfo> OpenToolSettings;

	TSharedPtr<FUICommandInfo> Refresh;
	TSharedPtr<FUICommandInfo> Apply;

	TSharedPtr<FUICommandInfo> ZoomToFit;

	TSharedPtr<FUICommandInfo> SetOperationToEaseOut;
	TSharedPtr<FUICommandInfo> SetOperationToEaseInOut;
	TSharedPtr<FUICommandInfo> SetOperationToEaseIn;

	TSharedPtr<FUICommandInfo> ToggleGridSnap;

	TSharedPtr<FUICommandInfo> ToggleAutoFlipTangents;

	TSharedPtr<FUICommandInfo> ToggleAutoZoomToFit;

	TSharedPtr<FUICommandInfo> ResetTangents;
	TSharedPtr<FUICommandInfo> ResetStartTangent;
	TSharedPtr<FUICommandInfo> ResetEndTangent;

	TSharedPtr<FUICommandInfo> FlattenTangents;
	TSharedPtr<FUICommandInfo> FlattenStartTangent;
	TSharedPtr<FUICommandInfo> FlattenEndTangent;

	TSharedPtr<FUICommandInfo> StraightenTangents;
	TSharedPtr<FUICommandInfo> StraightenStartTangent;
	TSharedPtr<FUICommandInfo> StraightenEndTangent;

	TSharedPtr<FUICommandInfo> CopyTangents;
	TSharedPtr<FUICommandInfo> PasteTangents;

	TSharedPtr<FUICommandInfo> CreateExternalCurveAsset;

	TSharedPtr<FUICommandInfo> SetKeyInterpConstant;
	TSharedPtr<FUICommandInfo> SetKeyInterpLinear;
	TSharedPtr<FUICommandInfo> SetKeyInterpCubicAuto;
	TSharedPtr<FUICommandInfo> SetKeyInterpCubicSmartAuto;
	TSharedPtr<FUICommandInfo> SetKeyInterpCubicUser;
	TSharedPtr<FUICommandInfo> SetKeyInterpCubicBreak;
	TSharedPtr<FUICommandInfo> SetKeyInterpToggleWeighted;

	TSharedPtr<FUICommandInfo> QuickEase;
	TSharedPtr<FUICommandInfo> QuickEaseIn;
	TSharedPtr<FUICommandInfo> QuickEaseOut;

	TSharedPtr<FUICommandInfo> SaveCurveEditorSelection;
};
