// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolExtender.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolSettings.h"
#include "EaseCurveToolSidebarSection.h"
#include "EaseCurveToolUtils.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "SequencerSettings.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolExtender"

namespace UE::EaseCurveTool
{

bool FEaseCurveToolExtender::bFirstTimeSequencerCreated = false;

FEaseCurveToolExtender& FEaseCurveToolExtender::Get()
{
	static FEaseCurveToolExtender Instance;
	return Instance;
}

FEaseCurveToolExtender::FEaseCurveToolExtender()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

	if (ISequencerModule* const SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerCreatedHandle = SequencerModule->RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FEaseCurveToolExtender::OnSequencerCreated));
	}
}

FEaseCurveToolExtender::~FEaseCurveToolExtender()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);

	if (ISequencerModule* const SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->UnregisterOnSequencerCreated(SequencerCreatedHandle);
	}

	for (TPair<FName, FEaseCurveToolInstance>& ToolInstance : ToolInstances)
	{
		RemoveSidebarExtension(ToolInstance.Value);
	}
}

FEaseCurveToolExtender::FEaseCurveToolInstance* FEaseCurveToolExtender::FindOrAddToolInstance_Internal(
	const TSharedRef<ISequencer>& InSequencer)
{
	const FName NewToolId = GetToolInstanceId(*InSequencer);
	if (NewToolId.IsNone())
	{
		return nullptr;
	}

	FEaseCurveToolInstance& NewToolInstance = ToolInstances.FindOrAdd(NewToolId);
	NewToolInstance.ToolId = NewToolId;
	NewToolInstance.Instance = MakeShared<FEaseCurveTool>(InSequencer);
	NewToolInstance.WeakSequencer = InSequencer;
	NewToolInstance.SequencerClosedHandle = InSequencer->OnCloseEvent().AddRaw(this, &FEaseCurveToolExtender::OnSequencerClosed, NewToolId);

	AddSidebarExtension(NewToolInstance);

	return &NewToolInstance;
}

void FEaseCurveToolExtender::OnSequencerCreated(const TSharedRef<ISequencer> InSequencer)
{
	FEaseCurveToolInstance* const NewToolInstance = FindOrAddToolInstance_Internal(InSequencer);
	if (!NewToolInstance || !NewToolInstance->Instance.IsValid())
	{
		return;
	}

	// Since the ease curve tool instance doesn't get created until the sequence is created,
	// we check and show the tool tab here after we know the instance exists. Otherwise we will
	// crash on editor startup attempting to bind commands through the constructor.
	if (!bFirstTimeSequencerCreated)
	{
		const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>();
		if (ToolSettings && ToolSettings->IsToolVisible())
		{
			NewToolInstance->Instance->ShowHideToolTab(true);
		}
	}

	bFirstTimeSequencerCreated = true;
}

void FEaseCurveToolExtender::OnSequencerClosed(const TSharedRef<ISequencer> InSequencer, const FName InToolId)
{
	const FName ToolId = GetToolInstanceId(InSequencer);

	if (ToolId.IsNone() || !ToolInstances.Contains(ToolId))
	{
		return;
	}

	if (!ToolInstances[ToolId].WeakSequencer.IsValid()
		|| ToolInstances[ToolId].WeakSequencer.Pin().ToSharedRef() != InSequencer)
	{
		return;
	}

	if (ToolInstances[ToolId].SequencerClosedHandle.IsValid())
	{
		InSequencer->OnCloseEvent().Remove(ToolInstances[ToolId].SequencerClosedHandle);
	}

	if (ToolInstances[ToolId].SidebarExtender.IsValid())
	{
		if (ISequencerModule* const SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
		{
			SequencerModule->GetSidebarExtensibilityManager()->RemoveExtender(ToolInstances[ToolId].SidebarExtender);
		}
		ToolInstances[ToolId].SidebarExtender.Reset();
	}

	RemoveSidebarExtension(ToolInstances[ToolId]);

	ToolInstances.Remove(ToolId);
}

FName FEaseCurveToolExtender::GetModularFeatureName()
{
	static FName FeatureName = TEXT("EaseCurveTool");
	return FeatureName;
}

FName FEaseCurveToolExtender::GetToolInstanceId(const ISequencer& InSequencer)
{
	const USequencerSettings* const SequencerSettings = InSequencer.GetSequencerSettings();
	return SequencerSettings ? SequencerSettings->GetFName() : NAME_None;
}

FName FEaseCurveToolExtender::GetToolInstanceId(const FEaseCurveTool& InTool)
{
	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		return GetToolInstanceId(*Sequencer);
	}
	return NAME_None;
}

TSharedPtr<FEaseCurveTool> FEaseCurveToolExtender::GetToolInstance(const FName InToolId)
{
	if (FEaseCurveToolInstance* const ToolInstance = Get().ToolInstances.Find(InToolId))
	{
		return ToolInstance->Instance;
	}
	return nullptr;
}

TSharedPtr<FEaseCurveTool> FEaseCurveToolExtender::FindToolInstance(const TSharedRef<ISequencer>& InSequencer)
{
	const FName ToolId = GetToolInstanceId(*InSequencer);
	if (!ToolId.IsNone())
	{
		return GetToolInstance(ToolId);
	}
	return nullptr;
}

TSharedPtr<FEaseCurveTool> FEaseCurveToolExtender::FindToolInstanceByCurveEditor(const TSharedRef<FCurveEditor>& InCurveEditor)
{
	for (const TPair<FName, FEaseCurveToolInstance>& ToolInstance : Get().ToolInstances)
	{
		if (const TSharedPtr<ISequencer> Sequencer = ToolInstance.Value.WeakSequencer.Pin())
		{
			if (const TSharedPtr<FCurveEditor> CurveEditor = FEaseCurveToolUtils::GetCurveEditorFromSequencer(Sequencer.ToSharedRef()))
			{
				if (CurveEditor == InCurveEditor)
				{
					return ToolInstance.Value.Instance;
				}
			}
		}
	}
	return nullptr;
}

void FEaseCurveToolExtender::AddSidebarExtension(FEaseCurveToolInstance& InToolInstance)
{
	check(InToolInstance.Instance.IsValid());

	InToolInstance.SidebarExtender = MakeShared<FExtender>();

	InToolInstance.SidebarExtender->AddMenuExtension(TEXT("KeyEdit")
		, EExtensionHook::After
		, InToolInstance.Instance->GetCommandList()
		, FMenuExtensionDelegate::CreateStatic(&FEaseCurveToolExtender::AddSidebarWidget, InToolInstance.ToolId)
	);

	if (ISequencerModule* const SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->GetSidebarExtensibilityManager()->AddExtender(InToolInstance.SidebarExtender);
	}
}

void FEaseCurveToolExtender::RemoveSidebarExtension(FEaseCurveToolInstance& InToolInstance)
{
	if (InToolInstance.SidebarExtender.IsValid())
	{
		if (ISequencerModule* const SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>(TEXT("Sequencer")))
		{
			SequencerModule->GetSidebarExtensibilityManager()->RemoveExtender(InToolInstance.SidebarExtender);
		}

		InToolInstance.SidebarExtender.Reset();
	}
}

void FEaseCurveToolExtender::AddSidebarWidget(FMenuBuilder& MenuBuilder, const FName InToolId)
{
	const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance(InToolId);
	if (!ToolInstance.IsValid())
	{
		return;
	}

	const TSharedRef<SExpandableArea> ToolMenuWidget =
		SNew(SExpandableArea)
		.AreaTitle(LOCTEXT("EaseCurveToolTitle", "Ease Curve Tool"))
		.InitiallyCollapsed(false)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
		.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.HeaderPadding(FMargin(2.f, 5.f))
		.Visibility_Static(&FEaseCurveToolExtender::GetSidebarVisibility, InToolId)
		.BodyContent()
		[
			ToolInstance->GenerateWidget()
		];

	FMenuEntryStyleParams StyleParams;
	StyleParams.bNoIndent = true;
	StyleParams.HorizontalAlignment = HAlign_Fill;
	StyleParams.VerticalAlignment = VAlign_Top;

	MenuBuilder.AddWidget(ToolMenuWidget
		, FText::GetEmpty()
		, StyleParams
		, /*bInSearchable=*/true
		, FText::GetEmpty()
		, FSlateIcon()
	);
}

EVisibility FEaseCurveToolExtender::GetSidebarVisibility(const FName InToolId)
{
	if (const TSharedPtr<FEaseCurveTool> ToolInstance = GetToolInstance(InToolId))
	{
		if (const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>())
		{
			return (ToolSettings->ShouldShowInSidebar()
					&& !ToolInstance->IsToolTabVisible()
					&& ToolInstance->HasCachedKeysToEase())
				? EVisibility::Visible : EVisibility::Collapsed;
		}
	}
	return EVisibility::Collapsed;
}

TSharedRef<SWidget> FEaseCurveToolExtender::MakeQuickPresetMenu(const FName InToolId)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	FEaseCurveToolInstance* const ToolInstance = Get().ToolInstances.Find(InToolId);
	if (!ToolInstance || !ToolInstance->Instance.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	constexpr const TCHAR* MenuName = TEXT("EaseCurveTool.QuickPreset");

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);

		const FEaseCurveToolCommands& EaseCurveToolCommands = FEaseCurveToolCommands::Get();

		FToolMenuSection& EaseCurveToolSection = ToolMenu->FindOrAddSection(TEXT("EaseCurveToolActions")
			, LOCTEXT("EaseCurveToolActions", "Ease Curve Tool"));

		EaseCurveToolSection.AddMenuEntry(EaseCurveToolCommands.QuickEase);
		EaseCurveToolSection.AddMenuEntry(EaseCurveToolCommands.QuickEaseIn);
		EaseCurveToolSection.AddMenuEntry(EaseCurveToolCommands.QuickEaseOut);
	}

	return ToolMenus->GenerateWidget(MenuName, FToolMenuContext(ToolInstance->Instance->GetCommandList()));
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
