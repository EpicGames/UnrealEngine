// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class FCurveEditor;
class FExtender;
class FMenuBuilder;
class FName;
class ISequencer;
class SWidget;
struct EVisibility;

#define UE_API EASECURVETOOL_API

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

/**
 * Singleton class for extending and managing tool instances.
 */
class FEaseCurveToolExtender : IModularFeature
{
public:
	UE_API static FEaseCurveToolExtender& Get();

	FEaseCurveToolExtender();
	virtual ~FEaseCurveToolExtender();

	static FName GetModularFeatureName();

	/**
	 * Retrieves the unique name for the provided Sequencer instance.
	 * The tool instance name is based on its associated sequencer settings
	 * object name. This provides a unique instance name for each Sequencer type.
	 * @param InSequencer The Sequencer instance for which the name is being retrieved.
	 * @return The name of the tool instance if available; otherwise, NAME_None.
	 */
	UE_API static FName GetToolInstanceId(const ISequencer& InSequencer);
	UE_API static FName GetToolInstanceId(const FEaseCurveTool& InTool);

	UE_API static TSharedPtr<FEaseCurveTool> GetToolInstance(const FName InToolId);

	/** Find the tool instance associated with the given Sequencer instance */
	UE_API static TSharedPtr<FEaseCurveTool> FindToolInstance(const TSharedRef<ISequencer>& InSequencer);

	UE_API static TSharedPtr<FEaseCurveTool> FindToolInstanceByCurveEditor(const TSharedRef<FCurveEditor>& InCurveEditor);

	static TSharedRef<SWidget> MakeQuickPresetMenu(const FName InToolId);

protected:
	struct FEaseCurveToolInstance
	{
		FName ToolId;

		TSharedPtr<FEaseCurveTool> Instance;

		TWeakPtr<ISequencer> WeakSequencer;

		FDelegateHandle SequencerClosedHandle;

		TSharedPtr<FExtender> SidebarExtender;
	};

	FEaseCurveToolInstance* FindOrAddToolInstance_Internal(const TSharedRef<ISequencer>& InSequencer);

	void OnSequencerCreated(const TSharedRef<ISequencer> InSequencer);
	void OnSequencerClosed(const TSharedRef<ISequencer> InSequencer, const FName InToolId);

	static void AddSidebarExtension(FEaseCurveToolInstance& InToolInstance);
	static void RemoveSidebarExtension(FEaseCurveToolInstance& InToolInstance);

	static void AddSidebarWidget(FMenuBuilder& MenuBuilder, const FName InToolId);

	static EVisibility GetSidebarVisibility(const FName InToolId);

private:
	static bool bFirstTimeSequencerCreated;

	FDelegateHandle SequencerCreatedHandle;

	/** Registered Sequencer types to the tool instances */
	TMap<FName, FEaseCurveToolInstance> ToolInstances;
};

} // namespace UE::EaseCurveTool

#undef UE_API
