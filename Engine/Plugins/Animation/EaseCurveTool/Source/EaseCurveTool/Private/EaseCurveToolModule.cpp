// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolModule.h"
#include "EaseCurveEditorExtension.h"
#include "EaseCurveStyle.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolExtender.h"
#include "ICurveEditorModule.h"
#include "Menus/EaseCurveLibraryMenu.h"
#include "ToolMenus.h"

void FEaseCurveToolModule::StartupModule()
{
	using namespace UE::EaseCurveTool;

	FEaseCurveToolCommands::Register();

	// Ensure singleton instances are created
	FEaseCurveStyle::Get();
	FEaseCurveToolExtender::Get();

	RegisterContentBrowserExtender();
	RegisterCurveEditorExtender();
}

void FEaseCurveToolModule::ShutdownModule()
{
	UnregisterCurveEditorExtender();
	UnregisterContentBrowserExtender();

	FEaseCurveToolCommands::Unregister();
}

void FEaseCurveToolModule::RegisterContentBrowserExtender()
{
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			UE::EaseCurveTool::FEaseCurveLibraryMenu::RegisterMenus();
		}));
	});
}

void FEaseCurveToolModule::UnregisterContentBrowserExtender()
{
	UE::EaseCurveTool::FEaseCurveLibraryMenu::UnregisterMenus();
}

void FEaseCurveToolModule::RegisterCurveEditorExtender()
{
	using namespace UE::EaseCurveTool;

	ICurveEditorModule* const CurveEditorModule = FModuleManager::GetModulePtr<ICurveEditorModule>(TEXT("CurveEditor"));
	if (!CurveEditorModule)
	{
		return;
	}

	CurveEditorExtenderHandle = CurveEditorModule->RegisterEditorExtension(
		FOnCreateCurveEditorExtension::CreateLambda([](const TWeakPtr<FCurveEditor> InCurveEditor)
		{
			return MakeShared<FEaseCurveEditorExtension>(InCurveEditor);
		}));
}

void FEaseCurveToolModule::UnregisterCurveEditorExtender()
{
	if (!CurveEditorExtenderHandle.IsValid())
	{
		return;
	}

	if (ICurveEditorModule* const CurveEditorModule = FModuleManager::GetModulePtr<ICurveEditorModule>(TEXT("CurveEditor")))
	{
		CurveEditorModule->UnregisterEditorExtension(CurveEditorExtenderHandle);
	}

	CurveEditorExtenderHandle.Reset();
}

IMPLEMENT_MODULE(FEaseCurveToolModule, EaseCurveTool)
