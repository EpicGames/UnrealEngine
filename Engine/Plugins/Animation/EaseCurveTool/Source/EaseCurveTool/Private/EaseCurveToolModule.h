// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleManager.h"

class FEaseCurveToolModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	void RegisterContentBrowserExtender();
	void UnregisterContentBrowserExtender();

	void RegisterCurveEditorExtender();
	void UnregisterCurveEditorExtender();

	FDelegateHandle CurveEditorExtenderHandle;
};
