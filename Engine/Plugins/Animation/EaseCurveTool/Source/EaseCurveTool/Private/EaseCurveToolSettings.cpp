// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolSettings.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolSettings"

UEaseCurveToolSettings::UEaseCurveToolSettings()
{
	CategoryName = TEXT("Sequencer");
	SectionName = TEXT("Ease Curve Tool");

	DefaultPresetLibrary = FSoftObjectPath(TEXT("/EaseCurveTool/DefaultPresetLibrary.DefaultPresetLibrary"));

	NewPresetCategory = LOCTEXT("NewPresetCategory", "Custom");
}

#undef LOCTEXT_NAMESPACE
