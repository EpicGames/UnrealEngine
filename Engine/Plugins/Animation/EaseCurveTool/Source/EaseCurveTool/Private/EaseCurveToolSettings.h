// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "EaseCurveLibrary.h"
#include "Engine/DeveloperSettings.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "EaseCurveToolSettings.generated.h"

UCLASS(config=EditorPerProjectUserSettings, meta = (DisplayName = "Ease Curve Tool"))
class UEaseCurveToolSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UEaseCurveToolSettings();

	bool ShouldShowInSidebar() const { return bShowInSidebar; }
	void SetShowInSidebar(const bool bInSpawnInTabWindow) { bShowInSidebar = bInSpawnInTabWindow; }
	void ToggleShowInSidebar() { bShowInSidebar = !bShowInSidebar; }

	bool IsToolVisible() const { return bToolTabVisible; }
	void SetToolVisible(const bool bInVisible) { bToolTabVisible = bInVisible; }

	bool ShouldShowCurveEditorToolbar() const { return bShowCurveEditorToolbarButton; }
	void SetShowCurveEditorToolbar(const bool bInShowCurveEditorToolbarButton) { bShowCurveEditorToolbarButton = bInShowCurveEditorToolbarButton; }
	void ToggleShowCurveEditorToolbar() { bShowCurveEditorToolbarButton = !bShowCurveEditorToolbarButton; }

	TObjectPtr<UEaseCurveLibrary> GetPresetLibrary() const { return DefaultPresetLibrary.LoadSynchronous(); }
	void SetPresetLibrary(const TObjectPtr<UEaseCurveLibrary>& InWeakPresetLibrary) { DefaultPresetLibrary = InWeakPresetLibrary; }

	FText GetNewPresetCategory() const { return NewPresetCategory; }
	void SetNewPresetCategory(const FText& InNewPresetCategory) { NewPresetCategory = InNewPresetCategory; }

	FString GetQuickEaseTangents() const { return QuickEaseTangents; }
	void SetQuickEaseTangents(const FString& InTangents) { QuickEaseTangents = InTangents; }

	int32 GetGraphSize() const { return GraphSize; }
	void SetGraphSize(const int32 InSize) { GraphSize = InSize; }

	bool GetGridSnap() const { return bGridSnap; }
	void SetGridSnap(const bool bInGridSnap) { bGridSnap = bInGridSnap; }
	void ToggleGridSnap() { bGridSnap = !bGridSnap; }

	int32 GetGridSize() const { return GridSize; }
	void SetGridSize(const int32 InSize) { GridSize = InSize; }

	bool GetAutoZoomToFit() const { return bAutoZoomToFit; }
	void SetAutoZoomToFit(const bool bInAutoZoomToFit) { bAutoZoomToFit = bInAutoZoomToFit; }
	void ToggleAutoZoomToFit() { bAutoZoomToFit = !bAutoZoomToFit; }

	bool GetAutoFlipTangents() const { return bAutoFlipTangents; }
	void SetAutoFlipTangents(const bool bInAutoFlipTangents) { bAutoFlipTangents = bInAutoFlipTangents; }
	void ToggleAutoFlipTangents() { bAutoFlipTangents = !bAutoFlipTangents; }

private:
	/** If true, displays the tool in the Sequencer sidebar drawer */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	bool bShowInSidebar = true;

	/** If true, shows the ease curve combo button in the curve editor toolbar */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	bool bShowCurveEditorToolbarButton = false;

	UPROPERTY(Config)
	bool bToolTabVisible = false;

	/** The preset library to use by default and saved when a preset library is changed from a dropdown */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	TSoftObjectPtr<UEaseCurveLibrary> DefaultPresetLibrary;

	/** The name of the category to place newly created curve presets. */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	FText NewPresetCategory;

	/** The tangents to apply for quick ease. Should be in the format of four comma-separated cubic bezier points. Ex. "0.45, 0.34, 0.0, 1.00" */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	FString QuickEaseTangents;

	/** The height of the curve ease tool in the details panel. */
	UPROPERTY(Config, EditAnywhere, Category = "Graph Editor", meta = (UIMin = 64, ClampMin = 64, UIMax = 256, ClampMax = 256, Delta = 1))
	int32 GraphSize = 140;

	/** If true, snaps tangents to grid. */
	UPROPERTY(Config, EditAnywhere, Category = "Graph Editor")
	bool bGridSnap = false;

	/** The height of the curve ease tool in the details panel. */
	UPROPERTY(Config, EditAnywhere, Category = "Graph Editor", meta = (UIMin = 4, ClampMin = 4, UIMax = 24, ClampMax = 24, Delta = 1))
	int32 GridSize = 8;

	/** If true, will auto zoom the graph editor to fit the tangent handles after they have been changed. */
	UPROPERTY(Config, EditAnywhere, Category = "Graph Editor")
	bool bAutoZoomToFit = false;

	/** If true, auto flips tangents when sequential key frame curve values are descending. */
	UPROPERTY(Config, EditAnywhere, Category = "Graph Editor")
	bool bAutoFlipTangents = true;
};
