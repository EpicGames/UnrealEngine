// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolSidebarSection.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolExtender.h"
#include "EaseCurveToolSettings.h"
#include "ISequencer.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolSidebarSection"

namespace UE::EaseCurveTool
{

const FName FEaseCurveToolSidebarSection::UniqueId = TEXT("EaseCurveToolSidebarSection");

FEaseCurveToolSidebarSection::FEaseCurveToolSidebarSection(const TSharedRef<ISequencer>& InSequencer)
	: WeakSequencer(InSequencer)
{
}

FName FEaseCurveToolSidebarSection::GetUniqueId() const
{
	return UniqueId;
}

FName FEaseCurveToolSidebarSection::GetSectionId() const
{
	return TEXT("Selection");
}

FText FEaseCurveToolSidebarSection::GetSectionDisplayText() const
{
	return LOCTEXT("EaseCurveToolLabel", "Ease Curve Tool");
}

bool FEaseCurveToolSidebarSection::ShouldShowSection() const
{
	if (const UEaseCurveToolSettings* const EaseCurveToolSettings = GetDefault<UEaseCurveToolSettings>())
	{
		if (!EaseCurveToolSettings->ShouldShowInSidebar())
		{
			return false;
		}

		if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			if (const TSharedPtr<FEaseCurveTool> ToolInstance = FEaseCurveToolExtender::Get().FindToolInstance(Sequencer.ToSharedRef()))
			{
				return !ToolInstance->IsToolTabVisible();
			}
		}
	}
	return false;
}

int32 FEaseCurveToolSidebarSection::GetSortOrder() const
{
	return -1;
}

TSharedRef<SWidget> FEaseCurveToolSidebarSection::CreateContentWidget()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (const TSharedPtr<FEaseCurveTool> Tool = FEaseCurveToolExtender::FindToolInstance(Sequencer.ToSharedRef()))
		{
			return Tool->GenerateWidget();
		}
	}
	return SNullWidget::NullWidget;
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
