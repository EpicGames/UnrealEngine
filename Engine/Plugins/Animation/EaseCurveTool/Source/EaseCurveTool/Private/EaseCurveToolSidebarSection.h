// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sidebar/ISidebarDrawerContent.h"
#include "Templates/SharedPointer.h"

class FName;
class FText;
class ISequencer;
class SWidget;

namespace UE::EaseCurveTool
{

class FEaseCurveToolSidebarSection : public ISidebarDrawerContent
{
public:
	static const FName UniqueId;

	FEaseCurveToolSidebarSection(const TSharedRef<ISequencer>& InSequencer);

	//~ Begin ISidebarDrawerContent
	virtual FName GetUniqueId() const override;
	virtual FName GetSectionId() const override;
	virtual FText GetSectionDisplayText() const override;
	virtual bool ShouldShowSection() const override;
	virtual int32 GetSortOrder() const override;
	virtual TSharedRef<SWidget> CreateContentWidget() override;
	//~ End ISidebarDrawerContent

protected:
	TWeakPtr<ISequencer> WeakSequencer;
};

} // namespace UE::EaseCurveTool
