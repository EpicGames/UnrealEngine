// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolTab.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolSettings.h"
#include "Editor.h"
#include "Framework/Docking/TabManager.h"
#include "ISequencer.h"
#include "Toolkits/IToolkitHost.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolTab"

namespace UE::EaseCurveTool
{

FEaseCurveToolTab::FEaseCurveToolTab(FEaseCurveTool& InOwnerTool)
	: OwnerTool(InOwnerTool)
{
}

void FEaseCurveToolTab::Init()
{
	bShuttingDown = false;

	RegisterToolTab();
}

void FEaseCurveToolTab::Shutdown()
{
	bShuttingDown = true;

	CloseToolTab();
	UnregisterToolTab();
}

bool FEaseCurveToolTab::ShouldShowToolTab() const
{
	if (const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>())
	{
		return ToolSettings->IsToolVisible();
	}
	return false;
}

bool FEaseCurveToolTab::IsToolTabVisible() const
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		const TSharedPtr<SDockTab> LiveTab = TabManager->FindExistingLiveTab(ToolTabId);
		return LiveTab.IsValid();
	}
	return false;
}

void FEaseCurveToolTab::ShowHideToolTab(const bool bInVisible)
{
	if (bInVisible)
	{
		if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
		{
			TabManager->TryInvokeTab(ToolTabId);
		}
	}
	else
	{
		CloseToolTab();
	}
}

void FEaseCurveToolTab::ToggleToolTabVisible()
{
	ShowHideToolTab(!IsToolTabVisible());
}

FName FEaseCurveToolTab::StaticToolTabId(const ISequencer& InSequencer)
{
	const FString DefaultTabId = TEXT("EaseCurveTool");

	if (const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>())
	{
		const FString TabName = DefaultTabId + TEXT(".") + ToolSettings->GetName();
		return *TabName;
	}

	return *DefaultTabId;
}

TSharedPtr<IToolkitHost> FEaseCurveToolTab::GetToolkitHost() const
{
	if (const TSharedPtr<ISequencer> SequencerHost = OwnerTool.GetSequencer())
	{
		return SequencerHost->GetToolkitHost();
	}
	return nullptr;
}

TSharedPtr<FTabManager> FEaseCurveToolTab::GetTabManager() const
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		return ToolkitHost->GetTabManager();
	}
	return nullptr;
}

USequencerSettings* FEaseCurveToolTab::GetSequencerSettings() const
{
	if (const TSharedPtr<ISequencer> SequencerHost = OwnerTool.GetSequencer())
	{
		return SequencerHost->GetSequencerSettings();
	}
	return nullptr;
}

void FEaseCurveToolTab::RegisterToolTab()
{
	static bool bAlreadyDoneOnce = false;

	if (bAlreadyDoneOnce || !ToolTabId.IsNone())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = OwnerTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	if (TabManager->HasTabSpawner(ToolTabId))
	{
		return;
	}

	ToolTabId = StaticToolTabId(*Sequencer);

	TabManager->RegisterTabSpawner(ToolTabId, FOnSpawnTab::CreateSP(this, &FEaseCurveToolTab::SpawnToolTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
		.SetDisplayName(LOCTEXT("EaseCurveToolTab", "Ease Curve Tool"))
		.SetIcon(FSlateIcon(FEaseCurveStyle::Get().GetStyleSetName(), TEXT("Icon.Tab")));
	bAlreadyDoneOnce = true;
}

void FEaseCurveToolTab::UnregisterToolTab()
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		TabManager->UnregisterTabSpawner(ToolTabId);
	}
}

TSharedRef<SDockTab> FEaseCurveToolTab::SpawnToolTab(const FSpawnTabArgs& InArgs)
{
	const TSharedRef<SDockTab> ToolTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.OnTabClosed_Lambda([this](const TSharedRef<SDockTab> InDockTab)
			{
				UEaseCurveToolSettings* const ToolSettings = GetMutableDefault<UEaseCurveToolSettings>();
				if (ToolSettings && !bShuttingDown)
				{
					ToolSettings->SetToolVisible(false);
				}

				VisibilityChangedDelegate.Broadcast(false);
			})
		[
			SNew(SBox)
			.MinDesiredWidth(100.f)
			.MinDesiredHeight(100.f)
			[
				OwnerTool.GenerateWidget()
			]
		];

	if (UEaseCurveToolSettings* const ToolSettings = GetMutableDefault<UEaseCurveToolSettings>())
	{
		ToolSettings->SetToolVisible(true);
	}

	VisibilityChangedDelegate.Broadcast(true);

	return ToolTab;
}

void FEaseCurveToolTab::CloseToolTab()
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	const TSharedPtr<SDockTab> LiveTab = TabManager->FindExistingLiveTab(ToolTabId);
	if (!LiveTab.IsValid())
	{
		return;
	}

	LiveTab->RequestCloseTab();
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
