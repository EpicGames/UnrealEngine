// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"

class FName;
class FTabManager;
class ISequencer;
class IToolkitHost;
class USequencerSettings;

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

class FEaseCurveToolTab : public TSharedFromThis<FEaseCurveToolTab>
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVisibilityChanged, const bool /*bInVisible*/);

	/** @return Returns the auto generated tool tab id using the passed in ISequencer.
	 * Tool tab ids are based on the sequencer instance settings name. */
	static FName StaticToolTabId(const ISequencer& InSequencer);

	FEaseCurveToolTab(FEaseCurveTool& InOwnerTool);

	void Init();
	void Shutdown();

	/** @return True if the USequencerSettings setting for the tool is set to visible */
	bool ShouldShowToolTab() const;

	/** @return True if the tool tab is visible */
	bool IsToolTabVisible() const;

	/** Shows or hides the tool tab and toggles the state in the saved settings */
	void ShowHideToolTab(const bool bInVisible);

	void ToggleToolTabVisible();

	FOnVisibilityChanged& OnVisibilityChanged() { return VisibilityChangedDelegate; }

protected:
	TSharedPtr<IToolkitHost> GetToolkitHost() const;
	TSharedPtr<FTabManager> GetTabManager() const;
	USequencerSettings* GetSequencerSettings() const;

	void RegisterToolTab();
	void UnregisterToolTab();

	TSharedRef<SDockTab> SpawnToolTab(const FSpawnTabArgs& InArgs);
	void CloseToolTab();

	FEaseCurveTool& OwnerTool;

	FName ToolTabId;

	/** State variable to help the dock tab know when it should save its visibility */
	bool bShuttingDown = false;

	FOnVisibilityChanged VisibilityChangedDelegate;
};

} // namespace UE::EaseCurveTool
