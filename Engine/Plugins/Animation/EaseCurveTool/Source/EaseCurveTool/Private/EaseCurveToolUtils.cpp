// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolUtils.h"
#include "CurveEditorSelection.h"
#include "EaseCurveTool.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

namespace UE::EaseCurveTool
{

UEaseCurveLibrary* FEaseCurveToolUtils::GetToolPresetLibrary(const TWeakPtr<FEaseCurveTool>& InWeakTool)
{
	if (const TSharedPtr<FEaseCurveTool> Tool = InWeakTool.Pin())
	{
		return Tool->GetPresetLibrary();
	}
	return nullptr;
}

bool FEaseCurveToolUtils::CompareCurveEditorSelections(const FCurveEditorSelection& InSelectionA, const FCurveEditorSelection& InSelectionB)
{
	const TMap<FCurveModelID, FKeyHandleSet> SelectionMapA = InSelectionA.GetAll();
	const TMap<FCurveModelID, FKeyHandleSet> SelectionMapB = InSelectionB.GetAll();

	if (SelectionMapA.Num() != SelectionMapB.Num())
	{
		return false;
	}

	for (const TPair<FCurveModelID, FKeyHandleSet>& PairA : SelectionMapA)
	{
		const FKeyHandleSet* const KeySetB = SelectionMapB.Find(PairA.Key);
		if (!KeySetB || KeySetB->Num() != PairA.Value.Num())
		{
			return false;
		}

		for (const FKeyHandle& KeyHandle : PairA.Value.AsArray())
		{
			if (!KeySetB->Contains(KeyHandle, ECurvePointType::Key))
			{
				return false;
			}
		}
	}

	return true;
}

TSharedPtr<FCurveEditor> FEaseCurveToolUtils::GetCurveEditorFromSequencer(const TSharedRef<ISequencer>& InSequencer)
{
	using namespace Sequencer;

	if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer->GetViewModel())
	{
		if (const FViewModelPtr RootModel = ViewModel->GetRootModel())
		{
			if (FCurveEditorExtension* const CurveEditorExtension = ViewModel->CastDynamic<FCurveEditorExtension>())
			{
				return CurveEditorExtension->GetCurveEditor();
			}
		}
	}

	return nullptr;
}

bool FEaseCurveToolUtils::FindChannelHandleFromCurveModel(const FCurveModel& InCurveModel, FMovieSceneChannelHandle& OutChannelHandle)
{
	UMovieSceneSection* const Section = InCurveModel.GetOwningObjectOrOuter<UMovieSceneSection>();
	if (!Section)
	{
		return false;
	}

	const void* RawCurve = InCurveModel.GetCurve();
	if (!RawCurve)
	{
		return false;
	}

	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	const TWeakPtr<FMovieSceneChannelProxy> WeakProxy = ChannelProxy.AsWeak();

	for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
	{
		const TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			if (Channels[Index] == RawCurve)
			{
				OutChannelHandle = FMovieSceneChannelHandle(WeakProxy, Entry.GetChannelTypeName(), Index);
				return true;
			}
		}
	}

	return false;
}

bool FEaseCurveToolUtils::HasWeightedBrokenTangents(const FRichCurveKey& InKey)
{
	return InKey.TangentMode == RCTM_Break
		&& InKey.TangentWeightMode == RCTWM_WeightedBoth
		&& InKey.InterpMode == RCIM_Cubic;
}

bool FEaseCurveToolUtils::HasWeightedBrokenTangents(const FMovieSceneDoubleValue& InValue)
{
	return InValue.TangentMode == RCTM_Break
		&& InValue.Tangent.TangentWeightMode == RCTWM_WeightedBoth
		&& InValue.InterpMode == RCIM_Cubic;
}

bool FEaseCurveToolUtils::HasWeightedBrokenTangents(const FMovieSceneFloatValue& InValue)
{
	return InValue.TangentMode == RCTM_Break
		&& InValue.Tangent.TangentWeightMode == RCTWM_WeightedBoth
		&& InValue.InterpMode == RCIM_Cubic;
}

} // namespace UE::EaseCurveTool
