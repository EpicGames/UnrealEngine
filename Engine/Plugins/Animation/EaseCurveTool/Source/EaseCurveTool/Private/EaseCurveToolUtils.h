// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor.h"
#include "ISequencer.h"

class UEaseCurveLibrary;
struct FCurveEditorSelection;
struct FMovieSceneFloatValue;
struct FMovieSceneDoubleValue;
struct FRichCurveKey;

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

class FEaseCurveToolUtils
{
public:
	static UEaseCurveLibrary* GetToolPresetLibrary(const TWeakPtr<FEaseCurveTool>& InWeakTool);

	/** Returns True if the curve editor selections match */
	static bool CompareCurveEditorSelections(const FCurveEditorSelection& InSelectionA, const FCurveEditorSelection& InSelectionB);

	static TSharedPtr<FCurveEditor> GetCurveEditorFromSequencer(const TSharedRef<ISequencer>& InSequencer);

	static bool FindChannelHandleFromCurveModel(const FCurveModel& InCurveModel, FMovieSceneChannelHandle& OutChannelHandle);

	static bool HasWeightedBrokenTangents(const FRichCurveKey& InKey);
	static bool HasWeightedBrokenTangents(const FMovieSceneDoubleValue& InValue);
	static bool HasWeightedBrokenTangents(const FMovieSceneFloatValue& InValue);
};

} // namespace UE::EaseCurveTool
