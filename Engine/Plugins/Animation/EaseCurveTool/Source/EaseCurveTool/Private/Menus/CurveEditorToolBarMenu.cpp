// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorToolBarMenu.h"
#include "EaseCurvePreset.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolMenuContext.h"
#include "EngineAnalytics.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/SEaseCurveAddNew.h"
#include "Widgets/SEaseCurveLibraryComboBox.h"
#include "Widgets/SEaseCurvePresetList.h"

#define LOCTEXT_NAMESPACE "CurveEditorToolBarMenu"

namespace UE::EaseCurveTool
{

const FName FCurveEditorToolBarMenu::MenuName = TEXT("CurveEditor.EaseCurveTool");

FCurveEditorToolBarMenu::FCurveEditorToolBarMenu(const TWeakPtr<FEaseCurveTool>& InWeakTool)
	: WeakTool(InWeakTool)
{
}

TSharedRef<SWidget> FCurveEditorToolBarMenu::GenerateWidget()
{
	const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (UEaseCurveToolMenuContext* const Context = InMenu->FindContext<UEaseCurveToolMenuContext>())
				{
					Context->OnPopulateMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	UEaseCurveToolMenuContext* const ContextObject = NewObject<UEaseCurveToolMenuContext>();
	ContextObject->Init(WeakTool);
	ContextObject->OnPopulateMenu = UEaseCurveToolMenuContext::FOnPopulateMenu::CreateRaw(this, &FCurveEditorToolBarMenu::PopulateMenu);

	FToolMenuContext MenuContext(Tool->GetCommandList(), nullptr, ContextObject);
	return ToolMenus->GenerateWidget(MenuName, MenuContext);
}

void FCurveEditorToolBarMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<FEaseCurveTool> ToolInstance = WeakTool.Pin();
	if (!ToolInstance.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("EaseCurveTool"), LOCTEXT("EaseCurveToolActions", "Ease Curve Tool Actions"));

	Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("Library")
		, SNew(SEaseCurveLibraryComboBox, ToolInstance.ToSharedRef())
		, FText::GetEmpty()));

	const TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::CreateSP(this, &FCurveEditorToolBarMenu::GetVisibilityForLibrary);

	FToolMenuEntry& SeparatorEntry = Section.AddSeparator(NAME_None);
	SeparatorEntry.Visibility = VisibilityAttribute;

	FToolMenuEntry AddNewEntry = FToolMenuEntry::InitWidget(TEXT("AddNew")
		, SNew(SEaseCurveAddNew, ToolInstance.ToSharedRef())
		, FText::GetEmpty());
	AddNewEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus"));
	AddNewEntry.Visibility = VisibilityAttribute;
	Section.AddEntry(AddNewEntry);

	FToolMenuEntry& PresetListEntry = Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("PresetList")
		, SNew(SEaseCurvePresetList, ToolInstance.ToSharedRef())
			.DisplayRate(this, &FCurveEditorToolBarMenu::GetDisplayRate)
			.OnPresetChanged(this, &FCurveEditorToolBarMenu::HandlePresetChanged)
			.OnQuickPresetChanged(this, &FCurveEditorToolBarMenu::HandleQuickPresetChanged)
		, FText::GetEmpty()));
	PresetListEntry.Visibility = VisibilityAttribute;
}

FFrameRate FCurveEditorToolBarMenu::GetDisplayRate() const
{
	if (const TSharedPtr<FEaseCurveTool> ToolInstance = WeakTool.Pin())
	{
		return ToolInstance->GetDisplayRate();
	}
	return FFrameRate();
}

void FCurveEditorToolBarMenu::HandlePresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	if (!InPreset.IsValid())
	{
		return;
	}

	const TSharedPtr<FEaseCurveTool> ToolInstance = WeakTool.Pin();
	if (!ToolInstance.IsValid())
	{
		return;
	}

	if (!ToolInstance->HasCachedKeysToEase())
	{
		ToolInstance->SetEaseCurveTangents(FEaseCurveTangents()
			, EEaseCurveToolOperation::InOut
			, /*bInBroadcastUpdate=*/true
			, /*bInSetSequencerTangents=*/false);

		FEaseCurveTool::ShowNotificationMessage(LOCTEXT("EqualValueKeys", "No different key values!"));

		return;
	}

	ToolInstance->SetEaseCurveTangents(InPreset->Tangents
		, EEaseCurveToolOperation::InOut
		, /*bInBroadcastUpdate=*/true
		, /*bInSetSequencerTangents=*/true);

	ToolInstance->RecordPresetAnalytics(InPreset, TEXT("CurveEditorExtension"));
}

void FCurveEditorToolBarMenu::HandleQuickPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	
}

EVisibility FCurveEditorToolBarMenu::GetVisibilityForLibrary() const
{
	if (const TSharedPtr<FEaseCurveTool> ToolInstance = WeakTool.Pin())
	{
		return (ToolInstance->GetPresetLibrary() != nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
