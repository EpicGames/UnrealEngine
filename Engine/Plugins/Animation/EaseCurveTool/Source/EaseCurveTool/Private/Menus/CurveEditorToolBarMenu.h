// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;
class UToolMenu;
struct EVisibility;
struct FEaseCurvePreset;
struct FEaseCurveTangents;
struct FFrameRate;

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

class FCurveEditorToolBarMenu : public TSharedFromThis<FCurveEditorToolBarMenu>
{
public:
	static const FName MenuName;

	FCurveEditorToolBarMenu(const TWeakPtr<FEaseCurveTool>& InWeakTool);

	TSharedRef<SWidget> GenerateWidget();

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	FFrameRate GetDisplayRate() const;
	void HandlePresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset);
	void HandleQuickPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset);

	EVisibility GetVisibilityForLibrary() const;

	TWeakPtr<FEaseCurveTool> WeakTool;
};

} // namespace UE::EaseCurveTool
