// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveLibraryMenu.h"
#include "AssetRegistry/AssetData.h"
#include "ContentBrowserMenuContexts.h"
#include "EaseCurveLibrary.h"
#include "EaseCurveSerializer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "EaseCurveLibraryMenu"

namespace UE::EaseCurveTool
{

void FEaseCurveLibraryMenu::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

	UToolMenu* const AssetContextMenu = ContentBrowser::ExtendToolMenu_AssetContextMenu(UEaseCurveLibrary::StaticClass());
	if (!AssetContextMenu)
	{
		return;
	}

	FToolMenuSection& Section = AssetContextMenu->FindOrAddSection(TEXT("GetAssetActions"));
	Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			AddMenuImportExportSection(InSection);
		}));
}

void FEaseCurveLibraryMenu::UnregisterMenus()
{
	if (UToolMenus* const ToolMenus = UToolMenus::Get())
	{
		ToolMenus->UnregisterOwnerByName(UE_MODULE_NAME);
	}
}

void FEaseCurveLibraryMenu::AddMenuEntryForNoSerializers(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddMenuEntry(LOCTEXT("NoSerializersName", "No serializers found")
		, LOCTEXT("NoSerializersTooltip", "To enable import/export options, a custom serializer should be setup")
		, FSlateIcon()
		, FExecuteAction()
	);
}

bool FEaseCurveLibraryMenu::AddMenuEntryForSerializer(FMenuBuilder& InMenuBuilder
	, UEaseCurveSerializer& InSerializer
	, const bool bInImport
	, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	if (InSerializer.HasAnyFlags(RF_ClassDefaultObject)
		&& !InSerializer.GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
	{
		if (bInImport)
		{
			InMenuBuilder.AddMenuEntry(InSerializer.GetDisplayName(),
				InSerializer.GetDisplayTooltip(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Import")),
				FExecuteAction::CreateStatic(&FEaseCurveLibraryMenu::PromptForImport, &InSerializer, InWeakLibraryAssets)
			);
		}
		else
		{
			InMenuBuilder.AddMenuEntry(InSerializer.GetDisplayName(),
				InSerializer.GetDisplayTooltip(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Export")),
				FExecuteAction::CreateStatic(&FEaseCurveLibraryMenu::PromptForExport, &InSerializer, InWeakLibraryAssets)
			);
		}

		return true;
	}

	return false;
}

void FEaseCurveLibraryMenu::AddMenuImportExportSection(FToolMenuSection& InSection)
{
	const UContentBrowserAssetContextMenuContext* const Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
	if (!Context)
	{
		return;
	}

	TSet<TWeakObjectPtr<UEaseCurveLibrary>> WeakLibraryAssets;

	for (const FAssetData& SelectedAsset : Context->SelectedAssets)
	{
		if (SelectedAsset.GetClass() == UEaseCurveLibrary::StaticClass())
		{
			if (UEaseCurveLibrary* const LibraryAsset = Cast<UEaseCurveLibrary>(SelectedAsset.GetAsset()))
			{
				WeakLibraryAssets.Add(LibraryAsset);
			}
		}
	}

	if (WeakLibraryAssets.IsEmpty())
	{
		return;
	}

	InSection.AddSubMenu(TEXT("EaseCurveLibrary_Import")
		, LOCTEXT("Import", "Import")
		, LOCTEXT("ImportTooltip", "Import presets to this library from external file sources")
		, FNewMenuDelegate::CreateStatic(&FEaseCurveLibraryMenu::PopulateImportMenu, WeakLibraryAssets)
		, false
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Import")));

	InSection.AddSubMenu(TEXT("EaseCurveLibrary_Export")
		, LOCTEXT("Export", "Export")
		, LOCTEXT("ExportTooltip", "Export presets from this library to external file sources")
		, FNewMenuDelegate::CreateStatic(&FEaseCurveLibraryMenu::PopulateExportMenu, WeakLibraryAssets)
		, false
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Export")));

	{
		FToolUIAction SetToDefaultPresetsAction;
		SetToDefaultPresetsAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
			[WeakLibraryAssets](const FToolMenuContext& InContext)
			{
				SetToDefaultPresets(WeakLibraryAssets);
			});

		InSection.AddMenuEntry(TEXT("EaseCurveLibrary_SetToDefaultPresets")
			, LOCTEXT("SetToDefaultPresets", "Set to Default Presets")
			, LOCTEXT("SetToDefaultPresetsTooltip", "Removes all current presets and adds all default presets")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Import"))
			, SetToDefaultPresetsAction);
	}
}

void FEaseCurveLibraryMenu::PopulateImportMenu(FMenuBuilder& InMenuBuilder
	, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	InMenuBuilder.BeginSection(TEXT("EaseCurveLibrary_Import"), LOCTEXT("ImportSection", "Import"));

	bool bIsEmpty = true;

	for (TObjectIterator<UEaseCurveSerializer> SerializerIt(RF_NoFlags); SerializerIt; ++SerializerIt)
	{
		if (UEaseCurveSerializer* const Serializer = *SerializerIt)
		{
			if (Serializer->SupportsImport()
				&& AddMenuEntryForSerializer(InMenuBuilder, *Serializer, /*bInImport=*/true, InWeakLibraryAssets))
			{
				bIsEmpty = false;
			}
		}
	}

	// Add placeholder text to help the user understand they need to create a custom serializer to enable options in this menu
	if (bIsEmpty)
	{
		AddMenuEntryForNoSerializers(InMenuBuilder);
	}

	InMenuBuilder.EndSection();
}

void FEaseCurveLibraryMenu::PopulateExportMenu(FMenuBuilder& InMenuBuilder
	, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	InMenuBuilder.BeginSection(TEXT("EaseCurveLibrary_Export"), LOCTEXT("ExportSection", "Export"));

	bool bIsEmpty = true;

	for (TObjectIterator<UEaseCurveSerializer> SerializerIt(RF_NoFlags); SerializerIt; ++SerializerIt)
	{
		if (UEaseCurveSerializer* const Serializer = *SerializerIt)
		{
			if (Serializer->SupportsExport()
				&& AddMenuEntryForSerializer(InMenuBuilder, *Serializer, /*bInImport=*/false, InWeakLibraryAssets))
			{
				bIsEmpty = false;
			}
		}
	}

	// Add placeholder text to help the user understand they need to create a custom serializer to enable options in this menu
	if (bIsEmpty)
	{
		AddMenuEntryForNoSerializers(InMenuBuilder);
	}

	InMenuBuilder.EndSection();
}

void FEaseCurveLibraryMenu::PromptForImport(UEaseCurveSerializer* const InSerializer
	, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	if (!InSerializer || !InSerializer->SupportsImport())
	{
		return;
	}

	FString FilePath;

	// Check for cancel if using file prompt
	if (InSerializer->IsFileImport()
		&& !InSerializer->PromptUserForFilePath(FilePath, /*bInImport=*/true))
	{
		return;
	}

	InSerializer->Import(FilePath, InWeakLibraryAssets);
}

void FEaseCurveLibraryMenu::PromptForExport(UEaseCurveSerializer* const InSerializer
	, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	if (!InSerializer || !InSerializer->SupportsExport())
	{
		return;
	}

	FString FilePath;

	// Check for cancel if using file prompt
	if (InSerializer->IsFileExport()
		&& !InSerializer->PromptUserForFilePath(FilePath, /*bInImport=*/false))
	{
		return;
	}

	InSerializer->Export(FilePath, InWeakLibraryAssets);
}

void FEaseCurveLibraryMenu::SetToDefaultPresets(const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets)
{
	TSet<UEaseCurveLibrary*> LibraryAssets;

	for (const TWeakObjectPtr<UEaseCurveLibrary>& WeakLibraryAsset : InWeakLibraryAssets)
	{
		if (WeakLibraryAsset.IsValid())
		{
			WeakLibraryAsset->SetToDefaultPresets();
		}
	}
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
