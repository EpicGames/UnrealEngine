// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"

class FMenuBuilder;
class UEaseCurveLibrary;
class UEaseCurveSerializer;
class UToolMenu;
struct FAssetData;
struct FToolMenuSection;

namespace UE::EaseCurveTool
{

class FEaseCurveLibraryMenu : public TSharedFromThis<FEaseCurveLibraryMenu>
{
public:
	static void RegisterMenus();
	static void UnregisterMenus();

protected:
	static void AddMenuImportExportSection(FToolMenuSection& InSection);

	static void AddMenuEntryForNoSerializers(FMenuBuilder& InMenuBuilder);

	static bool AddMenuEntryForSerializer(FMenuBuilder& InMenuBuilder
		, UEaseCurveSerializer& InSerializer
		, const bool bInImport
		, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);

	static void PopulateImportMenu(FMenuBuilder& InMenuBuilder
		, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);
	static void PopulateExportMenu(FMenuBuilder& InMenuBuilder
		, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);

	static void PromptForImport(UEaseCurveSerializer* const InSerializer
		, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);
	static void PromptForExport(UEaseCurveSerializer* const InSerializer
		, const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);

	static void SetToDefaultPresets(const TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraryAssets);
};

} // namespace UE::EaseCurveTool
