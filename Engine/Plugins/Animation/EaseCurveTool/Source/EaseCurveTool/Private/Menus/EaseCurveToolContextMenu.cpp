// Copyright Epic Games, Inc. All Rights Reserved.

#include "EaseCurveToolContextMenu.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "EaseCurveToolContextMenu"

namespace UE::EaseCurveTool
{

template<typename NumericType>
void ExtractNumericMetadata(FProperty* InProperty
	, TOptional<NumericType>& OutMinValue
	, TOptional<NumericType>& OutMaxValue
	, TOptional<NumericType>& OutSliderMinValue
	, TOptional<NumericType>& OutSliderMaxValue
	, NumericType& OutSliderExponent
	, NumericType& OutDelta
	, float& OutShiftMultiplier
	, float& OutCtrlMultiplier
	, bool& OutSupportDynamicSliderMaxValue
	, bool& OutSupportDynamicSliderMinValue)
{
	const FString& MetaUIMinString = InProperty->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = InProperty->GetMetaData(TEXT("UIMax"));
	const FString& SliderExponentString = InProperty->GetMetaData(TEXT("SliderExponent"));
	const FString& DeltaString = InProperty->GetMetaData(TEXT("Delta"));
	const FString& ShiftMultiplierString = InProperty->GetMetaData(TEXT("ShiftMultiplier"));
	const FString& CtrlMultiplierString = InProperty->GetMetaData(TEXT("CtrlMultiplier"));
	const FString& SupportDynamicSliderMaxValueString = InProperty->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
	const FString& SupportDynamicSliderMinValueString = InProperty->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
	const FString& ClampMinString = InProperty->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = InProperty->GetMetaData(TEXT("ClampMax"));

	// If no UIMin/Max was specified then use the clamp string
	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

	NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
	NumericType ClampMax = TNumericLimits<NumericType>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
	}

	NumericType UIMin = TNumericLimits<NumericType>::Lowest();
	NumericType UIMax = TNumericLimits<NumericType>::Max();
	TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
	TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);

	OutSliderExponent = NumericType(1);

	if (SliderExponentString.Len())
	{
		TTypeFromString<NumericType>::FromString(OutSliderExponent, *SliderExponentString);
	}

	OutDelta = NumericType(0);

	if (DeltaString.Len())
	{
		TTypeFromString<NumericType>::FromString(OutDelta, *DeltaString);
	}

	OutShiftMultiplier = 10.f;
	if (ShiftMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(OutShiftMultiplier, *ShiftMultiplierString);
	}

	OutCtrlMultiplier = 0.1f;
	if (CtrlMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(OutCtrlMultiplier, *CtrlMultiplierString);
	}

	const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
	const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

	OutMinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
	OutMaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
	OutSliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
	OutSliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

	OutSupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
	OutSupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
}

FEaseCurveToolContextMenu::FEaseCurveToolContextMenu(const TWeakPtr<FUICommandList>& InCommandListWeak, const FEaseCurveToolOnGraphSizeChanged& InOnGraphSizeChanged)
	: CommandListWeak(InCommandListWeak), OnGraphSizeChanged(InOnGraphSizeChanged)
{
	const UEaseCurveToolSettings* const EaseCurveToolSettings = GetDefault<UEaseCurveToolSettings>();
	check(EaseCurveToolSettings);
	GraphSize = EaseCurveToolSettings->GetGraphSize();
}

TSharedRef<SWidget> FEaseCurveToolContextMenu::GenerateWidget()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	constexpr const TCHAR* MenuName = TEXT("EaseCurveToolMenu");

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = ToolMenu->FindOrAddSection(TEXT("EaseCurveTool"), LOCTEXT("EaseCurveToolActions", "Curve Ease Tool Actions"));

		const FEaseCurveToolCommands& EaseCurveToolCommands = FEaseCurveToolCommands::Get();

		Section.AddSubMenu(TEXT("Settings"),
			LOCTEXT("SettingsSubMenuLabel", "Settings"),
			FText::GetEmpty(),
			FNewToolMenuDelegate::CreateSP(this, &FEaseCurveToolContextMenu::PopulateContextMenuSettings),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));

		Section.AddSeparator(NAME_None);

		Section.AddMenuEntry(EaseCurveToolCommands.CreateExternalCurveAsset);

		Section.AddSeparator(NAME_None);

		Section.AddMenuEntry(EaseCurveToolCommands.CopyTangents);

		Section.AddMenuEntry(EaseCurveToolCommands.PasteTangents);

		Section.AddSeparator(NAME_None);

		Section.AddSubMenu(TEXT("StraightenTangents"),
			LOCTEXT("StraightenTangentsSubMenuLabel", "Straighten Tangents"),
			FText::GetEmpty(),
			FNewToolMenuDelegate::CreateLambda([&EaseCurveToolCommands](UToolMenu* InToolMenu)
				{
					FToolMenuSection& NewSection = InToolMenu->FindOrAddSection(TEXT("StraightenTangents"));
					NewSection.AddMenuEntry(EaseCurveToolCommands.StraightenTangents);
					NewSection.AddSeparator(NAME_None);
					NewSection.AddMenuEntry(EaseCurveToolCommands.StraightenStartTangent);
					NewSection.AddMenuEntry(EaseCurveToolCommands.StraightenEndTangent);
				}),
			false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.StraightenTangents")));

		Section.AddSubMenu(TEXT("FlattenTangents"),
			LOCTEXT("FlattenTangentsSubMenuLabel", "Flatten Tangents"),
			FText::GetEmpty(),
			FNewToolMenuDelegate::CreateLambda([&EaseCurveToolCommands](UToolMenu* InToolMenu)
				{
					FToolMenuSection& NewSection = InToolMenu->FindOrAddSection(TEXT("FlattenTangents"));
					NewSection.AddMenuEntry(EaseCurveToolCommands.FlattenTangents);
					NewSection.AddSeparator(NAME_None);
					NewSection.AddMenuEntry(EaseCurveToolCommands.FlattenStartTangent);
					NewSection.AddMenuEntry(EaseCurveToolCommands.FlattenEndTangent);
				}), 
			false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("GenericCurveEditor.FlattenTangents")));

		Section.AddSubMenu(TEXT("ResetTangents"),
			LOCTEXT("ResetTangentsSubMenuLabel", "Reset Tangents"),
			FText::GetEmpty(),
			FNewToolMenuDelegate::CreateLambda([&EaseCurveToolCommands](UToolMenu* InToolMenu)
				{
					FToolMenuSection& NewSection = InToolMenu->FindOrAddSection(TEXT("ResetTangents"));
					NewSection.AddMenuEntry(EaseCurveToolCommands.ResetTangents);
					NewSection.AddSeparator(NAME_None);
					NewSection.AddMenuEntry(EaseCurveToolCommands.ResetStartTangent);
					NewSection.AddMenuEntry(EaseCurveToolCommands.ResetEndTangent);
				}),
			false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

		Section.AddSeparator(NAME_None);

		/** @TODO: Only show these in single edit mode(?)
		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpConstant);

		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpLinear);

		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpCubicAuto);

		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpCubicSmartAuto);

		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpCubicUser);

		Section.AddMenuEntry(EaseCurveToolCommands.SetKeyInterpCubicBreak);

		Section.AddSeparator(NAME_None);*/

		Section.AddMenuEntry(EaseCurveToolCommands.SetOperationToEaseOut);

		Section.AddMenuEntry(EaseCurveToolCommands.SetOperationToEaseInOut);

		Section.AddMenuEntry(EaseCurveToolCommands.SetOperationToEaseIn);

		Section.AddSeparator(NAME_None);

		Section.AddMenuEntry(EaseCurveToolCommands.ToggleGridSnap);

		Section.AddMenuEntry(EaseCurveToolCommands.ZoomToFit);

		Section.AddSeparator(NAME_None);

		Section.AddMenuEntry(EaseCurveToolCommands.Refresh);

		Section.AddMenuEntry(EaseCurveToolCommands.Apply);
	}

	return ToolMenus->GenerateWidget(MenuName, FToolMenuContext(CommandListWeak.Pin()));
}

void FEaseCurveToolContextMenu::PopulateContextMenuSettings(UToolMenu* const InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const FEaseCurveToolCommands& EaseCurveToolCommands = FEaseCurveToolCommands::Get();

	FToolMenuSection& Section = InToolMenu->FindOrAddSection(TEXT("EaseCurveToolSettings"), LOCTEXT("EaseCurveToolSettingsActions", "Settings"));

	Section.AddMenuEntry(EaseCurveToolCommands.OpenToolSettings);

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(EaseCurveToolCommands.ToggleAutoFlipTangents);

	Section.AddSeparator(NAME_None);

	// Graph Size
	{
		FProperty* const GraphSizeProperty = UEaseCurveToolSettings::StaticClass()->FindPropertyByName(TEXT("GraphSize"));
		check(GraphSizeProperty);

		TOptional<int32> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
		int32 SliderExponent, Delta;
		float ShiftMultiplier = 10.f;
		float CtrlMultiplier = 0.1f;
		bool SupportDynamicSliderMaxValue = false;
		bool SupportDynamicSliderMinValue = false;
		ExtractNumericMetadata(GraphSizeProperty
			, MinValue, MaxValue
			, SliderMinValue, SliderMaxValue
			, SliderExponent, Delta
			, ShiftMultiplier
			, CtrlMultiplier
			, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

		const TSharedRef<SNumericEntryBox<int32>> GraphSizeWidget = SNew(SNumericEntryBox<int32>)
			.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
			.AllowSpin(true)
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(SliderMinValue)
			.MaxSliderValue(SliderMaxValue)
			.SliderExponent(SliderExponent)
			.Delta(Delta)
			.ShiftMultiplier(ShiftMultiplier)
			.CtrlMultiplier(CtrlMultiplier)
			.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
			.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
			.Value_Lambda([this]()
				{
					return GraphSize;
				})
			.OnValueChanged_Lambda([this](const int32 InNewValue)
				{
					GraphSize = InNewValue;
				
					OnGraphSizeChanged.ExecuteIfBound(InNewValue);
				})
			.OnValueCommitted_Lambda([this](const int32 InNewValue, ETextCommit::Type InCommitType)
				{
					GraphSize = InNewValue;
				
					OnGraphSizeChanged.ExecuteIfBound(InNewValue);

					UEaseCurveToolSettings* const EaseCurveToolSettings = GetMutableDefault<UEaseCurveToolSettings>();
					check(EaseCurveToolSettings);
					EaseCurveToolSettings->SetGraphSize(InNewValue);
					EaseCurveToolSettings->SaveConfig();
				});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("ToolSize"), GraphSizeWidget, LOCTEXT("ToolSizeLabel", "Tool Size"), true));
	}

	// Grid Size
	{
		FProperty* const GridSizeProperty = UEaseCurveToolSettings::StaticClass()->FindPropertyByName(TEXT("GridSize"));
		check(GridSizeProperty);

		TOptional<int32> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
		int32 SliderExponent, Delta;
		float ShiftMultiplier = 10.f;
		float CtrlMultiplier = 0.1f;
		bool SupportDynamicSliderMaxValue = false;
		bool SupportDynamicSliderMinValue = false;
		ExtractNumericMetadata(GridSizeProperty
			, MinValue, MaxValue
			, SliderMinValue, SliderMaxValue
			, SliderExponent, Delta
			, ShiftMultiplier
			, CtrlMultiplier
			, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

		auto SetEaseCurveToolGridSize = [](const int32 InNewValue)
			{
				UEaseCurveToolSettings* const EaseCurveToolSettings = GetMutableDefault<UEaseCurveToolSettings>();
				check(EaseCurveToolSettings);
				EaseCurveToolSettings->SetGridSize(InNewValue);
				EaseCurveToolSettings->SaveConfig();
			};

		const TSharedRef<SNumericEntryBox<int32>> GridSizeWidget = SNew(SNumericEntryBox<int32>)
			.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
			.AllowSpin(true)
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(SliderMinValue)
			.MaxSliderValue(SliderMaxValue)
			.SliderExponent(SliderExponent)
			.Delta(Delta)
			.ShiftMultiplier(ShiftMultiplier)
			.CtrlMultiplier(CtrlMultiplier)
			.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
			.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
			.Value_Lambda([this]()
				{
					return GetDefault<UEaseCurveToolSettings>()->GetGridSize();
				})
			.OnValueChanged_Lambda(SetEaseCurveToolGridSize)
			.OnValueCommitted_Lambda([SetEaseCurveToolGridSize](const int32 InNewValue, ETextCommit::Type InCommitType)
				{
					SetEaseCurveToolGridSize(InNewValue);
				});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("GridSize"), GridSizeWidget, LOCTEXT("GridSizeLabel", "Grid Size"), true));
	}

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(EaseCurveToolCommands.ToggleAutoZoomToFit);
}

}

#undef LOCTEXT_NAMESPACE
