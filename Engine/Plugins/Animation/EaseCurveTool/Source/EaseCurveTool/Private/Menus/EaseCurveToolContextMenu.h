// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class SWidget;
class UToolMenu;

namespace UE::EaseCurveTool
{

DECLARE_DELEGATE_OneParam(FEaseCurveToolOnGraphSizeChanged, const int32 /*InNewSize*/)

class FEaseCurveToolContextMenu : public TSharedFromThis<FEaseCurveToolContextMenu>
{
public:
	FEaseCurveToolContextMenu(const TWeakPtr<FUICommandList>& InCommandListWeak, const FEaseCurveToolOnGraphSizeChanged& InOnGraphSizeChanged);

	TSharedRef<SWidget> GenerateWidget();

protected:
	void PopulateContextMenuSettings(UToolMenu* const InToolMenu);

	TWeakPtr<FUICommandList> CommandListWeak;

	FEaseCurveToolOnGraphSizeChanged OnGraphSizeChanged;

	int32 GraphSize;
};

} // namespace UE::EaseCurveTool
