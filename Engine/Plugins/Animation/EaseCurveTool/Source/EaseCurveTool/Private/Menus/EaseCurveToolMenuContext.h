// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTool.h"
#include "Templates/SharedPointer.h"
#include "EaseCurveToolMenuContext.generated.h"

class UToolMenu;

UCLASS()
class UEaseCurveToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<UE::EaseCurveTool::FEaseCurveTool>& InWeakTool)
	{
		WeakTool = InWeakTool;
	}

	TSharedPtr<UE::EaseCurveTool::FEaseCurveTool> GetTool() const
	{
		return WeakTool.Pin();
	}

	DECLARE_DELEGATE_OneParam(FOnPopulateMenu, UToolMenu*);
	FOnPopulateMenu OnPopulateMenu;

protected:
	TWeakPtr<UE::EaseCurveTool::FEaseCurveTool> WeakTool;
};
