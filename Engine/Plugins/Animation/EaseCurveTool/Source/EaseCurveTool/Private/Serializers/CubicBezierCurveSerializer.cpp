// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serializers/CubicBezierCurveSerializer.h"
#include "CubicBezierPointEntryDialog.h"
#include "EaseCurveLibrary.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "CubicBezierCurveSerializer"

FText UCubicBezierCurveSerializer::GetDisplayName() const
{
	return LOCTEXT("CubicBezierCurve_Name", "Cubic Bezier Curve");
}

/** The display tooltip text to show for menu entries */
FText UCubicBezierCurveSerializer::GetDisplayTooltip() const
{
	return LOCTEXT("CubicBezierCurve_Tooltip", "Import a cubic bezier curve as a preset");
}

bool UCubicBezierCurveSerializer::IsFileExport() const
{
	return false;
}

bool UCubicBezierCurveSerializer::SupportsExport() const
{
	return false;
}

bool UCubicBezierCurveSerializer::Export(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries)
{
	return false;
}

bool UCubicBezierCurveSerializer::IsFileImport() const
{
	return false;
}

bool UCubicBezierCurveSerializer::SupportsImport() const
{
	return true;
}

bool UCubicBezierCurveSerializer::Import(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries)
{
	return UE::EaseCurveTool::FCubicBezierPointEntryDialog::Prompt(InWeakLibraries);
}

#undef LOCTEXT_NAMESPACE
