// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "EaseCurveSerializer.h"
#include "CubicBezierCurveSerializer.generated.h"

class UEaseCurveLibrary;
class FText;

#define UE_API EASECURVETOOL_API

/** Serializes cubic bezier coordinates to cubic hermite */
UCLASS(MinimalAPI)
class UCubicBezierCurveSerializer : public UEaseCurveSerializer
{
	GENERATED_BODY()

public:
	//~ Begin UEaseCurveSerializer

	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayTooltip() const override;

	virtual bool IsFileExport() const override;
	virtual bool SupportsExport() const override;
	virtual bool Export(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries) override;

	virtual bool IsFileImport() const override;
	virtual bool SupportsImport() const override;
    virtual bool Import(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries) override;

	//~ End UEaseCurveSerializer
};

#undef UE_API
