// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serializers/CubicBezierPointEntryDialog.h"
#include "DetailLayoutBuilder.h"
#include "EaseCurveLibrary.h"
#include "EaseCurvePreset.h"
#include "EaseCurveTangents.h"
#include "EaseCurveToolSettings.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SEaseCurveCategoryComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CubicBezierPointEntryDialog"

namespace UE::EaseCurveTool
{

bool FCubicBezierPointEntryDialog::Prompt(const TSet<TWeakObjectPtr<UEaseCurveLibrary>>& InWeakLibraries)
{
	if (!GEditor)
	{
		return false;
	}

	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("AddCubicBezierPreset", "Add Cubic Bezier Preset"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea);

	const TSharedRef<FCubicBezierPointEntryDialog> Dialog = MakeShared<FCubicBezierPointEntryDialog>();
	Dialog->WeakLibraries = InWeakLibraries;
	Dialog->Window = Window;

	Window->SetContent(Dialog->BuildContent());

	GEditor->EditorAddModalWindow(Window);

	return Dialog->bHasAdded;
}

TSharedRef<SWidget> FCubicBezierPointEntryDialog::ConstructLabel(const FText& InLabel)
{
	return SNew(STextBlock)
		.ColorAndOpacity(FStyleColors::Foreground)
		.Text(InLabel);
}

TSharedRef<SWidget> FCubicBezierPointEntryDialog::BuildContent()
{
	using SNumericVectorInputBox2D = SNumericVectorInputBox<double, UE::Math::TVector2<double>, 2>;

	static constexpr float Padding = 12.f;

	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, Padding, Padding, 3.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground)
			.Text(LOCTEXT("PresetCategory", "Preset Category"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, 0.f, Padding, 6.f)
		[
			SNew(SEaseCurveCategoryComboBox, WeakLibraries)
			.OnChanged_Lambda([this](const FText& InCategory, const ESelectInfo::Type InSelectInfo)
				{
					NewCategoryName = InCategory;
					bLastAddedSuccessfully = false;
				})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, 0.f, Padding, 3.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FStyleColors::Foreground)
			.Text(LOCTEXT("PresetName", "Preset Name"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, 0.f, Padding, Padding)
		[
			SNew(SEditableTextBox)
			.OnTextChanged_Lambda([this](const FText& InText)
				{
					NewPresetName = InText;
					bLastAddedSuccessfully = false;
				})
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InCommitType)
				{
					NewPresetName = InText;
					bLastAddedSuccessfully = false;
				})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, 0.f, Padding, 6.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, Padding, 0.f)
			[
				ConstructLabel(LOCTEXT("Point1", "Point 1"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SNumericVectorInputBox2D)
				.bColorAxisLabels(true)
				.AllowSpin(true)
				.X_Lambda([this]()
					{
						return Point1.X;
					})
				.Y_Lambda([this]()
					{
						return Point1.Y;
					})
				.OnXChanged_Lambda([this](const double InNewValue)
					{
						Point1.X = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnYChanged_Lambda([this](const double InNewValue)
					{
						Point1.Y = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnXCommitted_Lambda([this](const double InNewValue, const ETextCommit::Type)
					{
						Point1.X = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnYCommitted_Lambda([this](const double InNewValue, const ETextCommit::Type)
					{
						Point1.Y = InNewValue;
						bLastAddedSuccessfully = false;
					})
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding, 0.f, Padding, 2.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, Padding, 0.f)
			[
				ConstructLabel(LOCTEXT("Point2", "Point 2"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SNumericVectorInputBox2D)
				.bColorAxisLabels(true)
				.AllowSpin(true)
				.X_Lambda([this]()
					{
						return Point2.X;
					})
				.Y_Lambda([this]()
					{
						return Point2.Y;
					})
				.OnXChanged_Lambda([this](const double InNewValue)
					{
						Point2.X = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnYChanged_Lambda([this](const double InNewValue)
					{
						Point2.Y = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnXCommitted_Lambda([this](const double InNewValue, const ETextCommit::Type)
					{
						Point2.X = InNewValue;
						bLastAddedSuccessfully = false;
					})
				.OnYCommitted_Lambda([this](const double InNewValue, const ETextCommit::Type)
					{
						Point2.Y = InNewValue;
						bLastAddedSuccessfully = false;
					})
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(Padding, Padding, Padding, 0.f)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity_Lambda([this]()
				{
					return StatusColor;
				})
			.Text_Lambda([this]()
				{
					return StatusText;
				})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(Padding)
		.HAlign(HAlign_Right)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FMargin(4.f, 0.f))
			+ SUniformGridPanel::Slot(0, 0)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.IsEnabled(this, &FCubicBezierPointEntryDialog::CanAdd)
				.OnClicked(this, &FCubicBezierPointEntryDialog::HandleAddButtonClick)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddButton", "Add"))
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.OnClicked(this, &FCubicBezierPointEntryDialog::HandleCloseButtonClick)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CloseButton", "Close"))
				]
			]
		];
}

bool FCubicBezierPointEntryDialog::CanAdd() const
{
	if (bLastAddedSuccessfully || NewPresetName.IsEmpty())
	{
		return false;
	}

	const FText CategoryToCheck = GetNewCategoryName();
	if (CategoryToCheck.IsEmpty())
	{
		SetStatus(FStyleColors::Error, LOCTEXT("NoDefaultCategory", "No default category in tool settings!"));
		return false;
	}

	const FText DefaultCategoryName = GetDefaultCategoryName();

	bool bNoExistingCategory = false;
	bool bPresetNameExists = false;

	ForEachLibrary([this, &CategoryToCheck, &DefaultCategoryName, &bNoExistingCategory, &bPresetNameExists]
		(const UEaseCurveLibrary* const InLibrary)
		{
			if (!InLibrary->DoesPresetCategoryExist(CategoryToCheck)
				&& !CategoryToCheck.EqualToCaseIgnored(DefaultCategoryName))
			{
				bNoExistingCategory = true;
				return false;
			}

			if (InLibrary->DoesPresetExist(FEaseCurvePresetHandle(CategoryToCheck, NewPresetName)))
			{
				bPresetNameExists = true;
				return false;
			}

			return true;
		});

	if (bNoExistingCategory)
	{
		SetStatus(FStyleColors::Error, LOCTEXT("NoCategoryExists", "No category exists in library!"));
	}
	else if (bPresetNameExists)
	{
		SetStatus(FStyleColors::Error, LOCTEXT("PresetNameExists", "Preset name already exists in category!"));
	}
	else
	{
		SetStatus();
	}

	return !bNoExistingCategory && !bPresetNameExists;
}

FReply FCubicBezierPointEntryDialog::HandleAddButtonClick()
{
	if (!CanAdd())
	{
		return FReply::Handled();
	}

	const FText CategoryToAdd = GetNewCategoryName();

	ForEachLibrary([this, &CategoryToAdd](UEaseCurveLibrary* const InLibrary)
		{
			const FEaseCurvePreset NewPreset(CategoryToAdd, NewPresetName
				, FEaseCurveTangents::MakeFromCubicBezier({ Point1.X, Point1.Y, Point2.X, Point2.Y }));
			InLibrary->AddPreset(NewPreset);
			return true;
		});

	bHasAdded = true;
	bLastAddedSuccessfully = true;

	SetStatus(FStyleColors::Success, LOCTEXT("AddedSuccessfully", "Added successfully!"));
	ResetDialog();

	return FReply::Handled();
}

FReply FCubicBezierPointEntryDialog::HandleCloseButtonClick()
{
	if (Window.IsValid())
	{
		Window->RequestDestroyWindow();
	}
	return FReply::Handled();
}

void FCubicBezierPointEntryDialog::ForEachLibrary(const TFunction<bool(UEaseCurveLibrary*)> InFunctor) const
{
	for (const TWeakObjectPtr<UEaseCurveLibrary>& Library : WeakLibraries)
	{
		if (Library.IsValid())
		{
			if (!InFunctor(Library.Get()))
			{
				return;
			}
		}
	}
}

void FCubicBezierPointEntryDialog::SetStatus(const FSlateColor& InColor, const FText& InText) const
{
	StatusColor = InColor;
	StatusText = InText;
}

FText FCubicBezierPointEntryDialog::GetDefaultCategoryName() const
{
	if (const UEaseCurveToolSettings* const ToolSettings = GetDefault<UEaseCurveToolSettings>())
	{
		return ToolSettings->GetNewPresetCategory();
	}
	return FText::GetEmpty();
}

FText FCubicBezierPointEntryDialog::GetNewCategoryName() const
{
	return NewCategoryName.IsEmpty() ? GetDefaultCategoryName() : NewCategoryName;
}

void FCubicBezierPointEntryDialog::ResetDialog()
{
	NewPresetName = FText::GetEmpty();

	Point1 = FVector2d::ZeroVector;
	Point2 = FVector2d::ZeroVector;
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
