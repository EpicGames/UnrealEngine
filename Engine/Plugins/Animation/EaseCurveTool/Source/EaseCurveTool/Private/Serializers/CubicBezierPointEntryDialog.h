// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Math/MathFwd.h"
#include "Misc/OptionalFwd.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FReply;
class SWidget;
class SWindow;
class UEaseCurveLibrary;
class UObject;

namespace UE::EaseCurveTool
{

class FCubicBezierPointEntryDialog : public TSharedFromThis<FCubicBezierPointEntryDialog>
{
public:
	static bool Prompt(const TSet<TWeakObjectPtr<UEaseCurveLibrary>>& InWeakLibraries);

protected:
	TSharedRef<SWidget> ConstructLabel(const FText& InLabel);
	TSharedRef<SWidget> BuildContent();

	bool CanAdd() const;

	FReply HandleAddButtonClick();
	FReply HandleCloseButtonClick();

	void ForEachLibrary(const TFunction<bool(UEaseCurveLibrary*)> InFunctor) const;

	void SetStatus(const FSlateColor& InColor = FStyleColors::Foreground, const FText& InText = FText::GetEmpty()) const;

	FText GetDefaultCategoryName() const;

	/** @return The current selected category name.
	 * If the category dropdown is set to "(Default)", this will return the default category from the tool settings. */
	FText GetNewCategoryName() const;

	void ResetDialog();

	TSet<TWeakObjectPtr<UEaseCurveLibrary>> WeakLibraries;

	TSharedPtr<SWindow> Window;

	bool bHasAdded = false;

	FText NewCategoryName;
	FText NewPresetName;

	FVector2d Point1 = FVector2d::ZeroVector;
	FVector2d Point2 = FVector2d::ZeroVector;

	mutable FSlateColor StatusColor;
	mutable FText StatusText;
	bool bLastAddedSuccessfully = false;
};

} // namespace UE::EaseCurveTool
