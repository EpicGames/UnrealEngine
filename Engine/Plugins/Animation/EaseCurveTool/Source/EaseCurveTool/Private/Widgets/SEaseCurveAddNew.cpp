// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveAddNew.h"
#include "EaseCurveLibrary.h"
#include "EaseCurvePreset.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SEaseCurveAddNew"

namespace UE::EaseCurveTool
{

void SEaseCurveAddNew::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	WeakTool = InTool;

	ChildSlot
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]()
			{
				return bIsCreatingNewPreset ? 1 : 0;
			})
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(NewPresetButton, SButton)
			.ButtonStyle(FEaseCurveStyle::Get(), TEXT("MenuHoverHintOnly"))
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("AddNewPresetToolTip", "Save the current ease curve as a new preset"))
			.IsEnabled(this, &SEaseCurveAddNew::IsButtonEnabled)
			.OnClicked(this, &SEaseCurveAddNew::OnCreateNewPresetClick)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text(LOCTEXT("AddNewPresetText", "Add New Preset"))
			]
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(NewPresetNameTextBox, SEditableTextBox)
				.OnKeyDownHandler(this, &SEaseCurveAddNew::OnNewPresetKeyDownHandler)
				.OnTextCommitted(this, &SEaseCurveAddNew::OnNewPresetTextCommitted)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton"))
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("CancelNewPresetToolTip", "Cancels the current new ease curve preset operation"))
				.IsEnabled(this, &SEaseCurveAddNew::IsButtonEnabled)
				.OnClicked(this, &SEaseCurveAddNew::OnCancelNewPresetClick)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(FEaseCurveStyle::Get().GetFloat(TEXT("ToolButton.ImageSize"))))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(TEXT("Icons.X")))
				]
			]
		]
	];
}

bool SEaseCurveAddNew::IsButtonEnabled() const
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		FEaseCurveKeySelection& KeySelection = Tool->GetSelectedKeyCache();
		const int32 SelectedKeyCount = KeySelection.GetTotalSelectedKeys();
		const int32 KeyChannelCount = KeySelection.GetKeyChannelCount();
		const bool bValidKeyCount = KeyChannelCount == 1 && SelectedKeyCount > 0 && SelectedKeyCount < 3;
		return bValidKeyCount
			&& Tool->GetPresetLibrary()
			&& Tool->HasCachedKeysToEase()
			&& Tool->GetSelectionError() == EEaseCurveToolError::None;
	}
	return false;
}

FReply SEaseCurveAddNew::OnCreateNewPresetClick()
{
	bIsCreatingNewPreset = true;

	FSlateApplication::Get().SetAllUserFocus(NewPresetNameTextBox);

	return FReply::Handled();
}

FReply SEaseCurveAddNew::OnCancelNewPresetClick()
{
	bIsCreatingNewPreset = false;
	NewPresetNameTextBox->SetText(FText::GetEmpty());

	return FReply::Handled();
}

FReply SEaseCurveAddNew::OnNewPresetKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		bIsCreatingNewPreset = false;
		NewPresetNameTextBox->SetText(FText::GetEmpty());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SEaseCurveAddNew::OnNewPresetTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter
		&& !InNewText.IsEmpty()
		&& IsButtonEnabled())
	{
		if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
		{
			if (UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool))
			{
				if (const TSharedPtr<ISequencer> Sequencer = Tool->GetSequencer())
				{
					FEaseCurveKeySelection& KeySelection = Tool->GetSelectedKeyCache();
					const int32 SelectedKeyCount = KeySelection.GetTotalSelectedKeys();
					const int32 KeyChannelCount = KeySelection.GetKeyChannelCount();
					const bool bValidKeyCount = KeyChannelCount == 1 && SelectedKeyCount > 0 && SelectedKeyCount < 3;
					if (bValidKeyCount)
					{
						FEaseCurvePreset NewPreset;
						if (!PresetLibrary->AddPresetToNewCategory(InNewText, Tool->GetAverageTangentsFromKeyCache(), NewPreset))
						{
							FEaseCurveTool::ShowNotificationMessage(LOCTEXT("AlreadyExistsError", "Fail: Preset name already exists!"));
						}
					}
				}
			}
		}
	}

	bIsCreatingNewPreset = false;
	NewPresetNameTextBox->SetText(FText::GetEmpty());
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
