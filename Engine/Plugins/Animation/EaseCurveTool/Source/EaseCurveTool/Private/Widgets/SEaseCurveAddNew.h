// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTangents.h"
#include "Widgets/SCompoundWidget.h"

class SButton;
class SEditableTextBox;

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

class SEaseCurveAddNew : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetNewPresetTangents, FEaseCurveTangents& /*InTangents*/)

	SLATE_BEGIN_ARGS(SEaseCurveAddNew) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

protected:
	bool IsButtonEnabled() const;

	FReply OnCreateNewPresetClick();
	FReply OnCancelNewPresetClick();
	
	FReply OnNewPresetKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);
	void OnNewPresetTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType);

	TWeakPtr<FEaseCurveTool> WeakTool;

	TSharedPtr<SButton> NewPresetButton;
	TSharedPtr<SEditableTextBox> NewPresetNameTextBox;

	bool bIsCreatingNewPreset = false;
};

} // namespace UE::EaseCurveTool
