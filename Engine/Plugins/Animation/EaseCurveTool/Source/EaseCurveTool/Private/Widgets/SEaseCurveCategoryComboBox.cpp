// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveCategoryComboBox.h"
#include "EaseCurveLibrary.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SEaseCurveCategoryComboBox"

namespace UE::EaseCurveTool
{

void SEaseCurveCategoryComboBox::Construct(const FArguments& InArgs, const TWeakObjectPtr<UEaseCurveLibrary>& InWeakLibrary)
{
	Construct(InArgs, TSet<TWeakObjectPtr<UEaseCurveLibrary>>{ InWeakLibrary });
}

void SEaseCurveCategoryComboBox::Construct(const FArguments& InArgs, const TSet<TWeakObjectPtr<UEaseCurveLibrary>>& InWeakLibraries)
{
	WeakLibraries = InWeakLibraries;

	OnCategoryChanged = InArgs._OnChanged;

	if (InArgs._UseDefault)
	{
		DefaultCategory = MakeShared<FText>(LOCTEXT("DefaultCategory", "(Default)"));
		AllCategories.Add(DefaultCategory);

		SelectedCategory = DefaultCategory;
	}

	for (auto Iter = WeakLibraries.CreateIterator(); Iter; ++Iter)
	{
		const TWeakObjectPtr<UEaseCurveLibrary> Library = *Iter;
		if (Library.IsValid())
		{
			for (const FText& Category : Library->GetCategories())
			{
				AllCategories.Add(MakeShared<FText>(Category));
			}
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	AllCategories.Sort([this](const TSharedPtr<FText>& InA, const TSharedPtr<FText>& InB)
		{
			if (DefaultCategory.IsValid() && InA == DefaultCategory)
			{
				return true;
			}
			return InA->CompareToCaseIgnored(*InB) < 0;
		});

	ChildSlot
	[
		SNew(SComboBox<TSharedPtr<FText>>)
		.OptionsSource(&AllCategories)
		.OnGenerateWidget_Lambda([](const TSharedPtr<FText> InItem)
			{
				return SNew(STextBlock)
					.ColorAndOpacity(FStyleColors::Foreground)
					.Text(*InItem);
			})
		.OnSelectionChanged(this, &SEaseCurveCategoryComboBox::HandleCategoryChanged)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SEaseCurveCategoryComboBox::GetSelectedCategory)
		]
	];
}

FText SEaseCurveCategoryComboBox::GetSelectedCategory() const
{
	if (SelectedCategory.IsValid())
	{
		return *SelectedCategory;
	}
	return FText::GetEmpty();
}

TSharedPtr<FText> SEaseCurveCategoryComboBox::GetSelectedCategoryItem() const
{
	return SelectedCategory;
}

void SEaseCurveCategoryComboBox::HandleCategoryChanged(const TSharedPtr<FText> InNewCategory, const ESelectInfo::Type InSelectInfo)
{
	if (WeakLibraries.IsEmpty())
	{
		return;
	}

	SelectedCategory = InNewCategory;

	const FText CategoryName = (SelectedCategory == DefaultCategory) ? FText::GetEmpty() : *SelectedCategory;
	OnCategoryChanged.ExecuteIfBound(CategoryName, InSelectInfo);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
