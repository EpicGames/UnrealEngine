// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FText;
class UEaseCurveLibrary;

namespace UE::EaseCurveTool
{

/** A combo box widget to display and select a category from a list of all categories that exist inside a library  */
class SEaseCurveCategoryComboBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnCategoryChanged, const FText& /*InNewCategory*/, const ESelectInfo::Type /*InSelectInfo*/)

	SLATE_BEGIN_ARGS(SEaseCurveCategoryComboBox) {}
		SLATE_ARGUMENT(bool, UseDefault)
		SLATE_EVENT(FOnCategoryChanged, OnChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakObjectPtr<UEaseCurveLibrary>& InWeakLibrary);
	void Construct(const FArguments& InArgs, const TSet<TWeakObjectPtr<UEaseCurveLibrary>>& InWeakLibraries);

	FText GetSelectedCategory() const;

protected:
	TSharedPtr<FText> GetSelectedCategoryItem() const;
	void HandleCategoryChanged(const TSharedPtr<FText> InNewCategory, const ESelectInfo::Type InSelectInfo);

	TSet<TWeakObjectPtr<UEaseCurveLibrary>> WeakLibraries;

	FOnCategoryChanged OnCategoryChanged;

	TArray<TSharedPtr<FText>> AllCategories;

	TSharedPtr<FText> DefaultCategory;
	TSharedPtr<FText> SelectedCategory;
};

} // namespace UE::EaseCurveTool
