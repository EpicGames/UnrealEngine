// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveEditor.h"
#include "Curves/SimpleCurve.h"
#include "EaseCurveStyle.h"
#include "Editor.h"
#include "Factories/Factory.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Layout/WidgetPath.h"
#include "Rendering/DrawElements.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SEaseCurvePreview.h"
#include "Widgets/SEaseCurvePreviewToolTip.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SEaseCurveEditor"

namespace UE::EaseCurveTool
{

static const FVector2D KeyHitSize = FVector2D(8.f);
static const FVector2D TangentHitSize = FVector2D(20.f);
static const FVector2D TangentDrawSize = FVector2D(16.f);
static const FVector2D CurveHitSize = FVector2D(8.f);

static const FVector2D HalfKeyHitSize = KeyHitSize * 0.5f;
static const FVector2D HalfTangentHitSize = TangentHitSize * 0.5f;
static const FVector2D HalfTangentDrawSize = TangentDrawSize * 0.5f;
static const FVector2D HalfCurveHitSize = CurveHitSize * 0.5f;

static constexpr float FitMargin = 0.1f;
static constexpr float MinViewRange = 0.5f;
static constexpr float HalfMinViewRange = MinViewRange * 0.5f;

static constexpr float NormalBoundsThickness = 1.f;
static constexpr float TangentHandleLineThickness = 2.f;

FVector2d SEaseCurveEditor::CalcTangentDir(const double InTangent)
{
	const double Angle = FMath::Atan(InTangent);
	return FVector2d(FMath::Cos(Angle), FMath::Sin(Angle));
}

float SEaseCurveEditor::CalcTangent(const FVector2d& InHandleDelta)
{
	return InHandleDelta.Y / FMath::Max<double>(InHandleDelta.X, KINDA_SMALL_NUMBER);
}

void SEaseCurveEditor::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	Tool = InTool;
	EaseCurve = InTool->GetToolCurve();

	DataMinInput = InArgs._DataMinInput;
	DataMaxInput = InArgs._DataMaxInput;

	DesiredSize = InArgs._DesiredSize;
	DisplayRate = InArgs._DisplayRate;

	bShowInputGridNumbers = InArgs._ShowInputGridNumbers;
	bShowOutputGridNumbers = InArgs._ShowOutputGridNumbers;

	NormalAreaColor = InArgs._NormalAreaColor;
	NormalBoundsColor = InArgs._NormalBoundsColor;
	bGridSnap = InArgs._GridSnap;
	GridSize = InArgs._GridSize;
	GridColor = InArgs._GridColor;
	ExtendedGridColor = InArgs._ExtendedGridColor;
	CurveThickness = InArgs._CurveThickness;
	CurveColor = InArgs._CurveColor;
	Operation = InArgs._Operation;

	bCanEditCurve = InArgs._CanEditCurve;
	ErrorMessage = InArgs._ErrorMessage;

	StartText = InArgs._StartText;
	StartTooltipText = InArgs._StartTooltipText;
	EndText = InArgs._EndText;
	EndTooltipText = InArgs._EndTooltipText;
	
	OnTangentsChanged = InArgs._OnTangentsChanged;
	GetContextMenuContent = InArgs._GetContextMenuContent;
	OnKeyDownEvent = InArgs._OnKeyDown;
	OnDragStart = InArgs._OnDragStart;
	OnDragEnd = InArgs._OnDragEnd;

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SEaseCurveEditor::OnObjectPropertyChanged);

	ZoomToFit();

	ChildSlot
		[
			SNew(SColorBlock)
			.Color(FStyleColors::Background.GetSpecifiedColor())
		];
}

FText SEaseCurveEditor::GetCurveToolTipInputText() const
{
	return CurveToolTipInputText;
}

FText SEaseCurveEditor::GetCurveToolTipOutputText() const
{
	return CurveToolTipOutputText;
}

SEaseCurveEditor::~SEaseCurveEditor()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

FVector2D SEaseCurveEditor::ComputeDesiredSize(const float InLayoutScaleMultiplier) const
{
	const TOptional<FVector2D> SizeFromAttribute = DesiredSize.Get(TOptional<FVector2D>());
	return SizeFromAttribute.Get(FVector2D::ZeroVector);
}

int32 SEaseCurveEditor::OnPaint(const FPaintArgs& InArgs
	, const FGeometry& InGeometry
	, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const FWidgetStyle& InWidgetStyle
	, const bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InArgs, InGeometry, InCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	const bool bEnabled = ShouldBeEnabled(bInParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InGeometry.GetLocalSize());

	if (FMath::IsNearlyEqual(ViewMinInput, ViewMaxInput) || FMath::IsNearlyEqual(ViewMinOutput, ViewMaxOutput))
	{
		return InLayerId;
	}
	
	const float ZeroInputX = ScaleInfo.InputToLocalX(0.f);
	const float ZeroOutputY = ScaleInfo.OutputToLocalY(0.f);
	const float OneInputX = ScaleInfo.InputToLocalX(1.f);
	const float OneOutputY = ScaleInfo.OutputToLocalY(1.f);

	// Normalized area background
	{
		const float InputSize = OneInputX - ZeroInputX;
		const float OutputSize = (OneOutputY - ZeroOutputY) * -1;
		const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

		FSlateDrawElement::MakeBox(OutDrawElements, ++InLayerId,
			InGeometry.ToPaintGeometry(FVector2D(InputSize, OutputSize), FSlateLayoutTransform(FVector2D(ZeroInputX, OneOutputY))),
			WhiteBrush, DrawEffects, NormalAreaColor);
	}

	auto PaintCenteredText = [&InGeometry, &OutDrawElements, &InLayerId, DrawEffects, &ScaleInfo](const FText& InText)
	{
		const FSlateFontInfo FontInfo = FEaseCurveStyle::Get().GetFontStyle(TEXT("Editor.ErrorFont"));

		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FVector2D TextSize = FontMeasure->Measure(InText, FontInfo);

		const float HalfInputX = ScaleInfo.InputToLocalX(0.5f);
		const float HalfOutputY = ScaleInfo.OutputToLocalY(0.5f);

		const FVector2D HalfTextSize = TextSize * 0.5f;
		const FVector2D ActualOffset(HalfInputX - HalfTextSize.X, HalfOutputY - HalfTextSize.Y);

		FSlateDrawElement::MakeText(OutDrawElements, ++InLayerId
			, InGeometry.ToPaintGeometry(InGeometry.Size, FSlateLayoutTransform(ActualOffset))
			, InText, FontInfo, DrawEffects, FStyleColors::Foreground.GetSpecifiedColor());
	};

	if (CanEditCurve())
	{
		InLayerId = PaintGrid(ScaleInfo, InGeometry, OutDrawElements, ++InLayerId, InCullingRect, DrawEffects);

		if (StartText.IsSet() || EndText.IsSet())
		{
			const FSlateFontInfo FontInfo = FEaseCurveStyle::Get().GetFontStyle(TEXT("Editor.LabelFont"));
			constexpr FLinearColor TextColor = FLinearColor(0.1f, 0.1f, 0.1f, 1.f);
			const FVector2D TextOffset = FVector2D(4.f);

			if (StartText.IsSet())
			{
				const FText& Text = StartText.Get();
				const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D TextSize = FontMeasure->Measure(Text, FontInfo);

				const FVector2D ActualOffset(ZeroInputX - (TextOffset.X + TextSize.X), ZeroOutputY - (TextOffset.Y + TextSize.Y));

				FSlateDrawElement::MakeText(OutDrawElements, ++InLayerId
					, InGeometry.ToPaintGeometry(InGeometry.Size, FSlateLayoutTransform(ActualOffset))
					, Text, FontInfo, DrawEffects, TextColor);
			}

			if (EndText.IsSet())
			{
				const FVector2D ActualOffset(OneInputX + TextOffset.X, OneOutputY + TextOffset.Y);

				FSlateDrawElement::MakeText(OutDrawElements, ++InLayerId
					, InGeometry.ToPaintGeometry(InGeometry.Size, FSlateLayoutTransform(ActualOffset))
					, EndText.Get(), FontInfo, DrawEffects, TextColor);
			}
		}

		PaintNormalBounds(ScaleInfo, InGeometry, OutDrawElements, ++InLayerId, InCullingRect, DrawEffects, InWidgetStyle);

		PaintCurve(ScaleInfo, InGeometry, OutDrawElements, ++InLayerId, InCullingRect, DrawEffects, InWidgetStyle);

		InLayerId = PaintKeys(ScaleInfo, InGeometry, OutDrawElements, ++InLayerId, DrawEffects, InWidgetStyle);
	}

	if (HasDrawErrorMessage())
	{
		PaintCenteredText(ErrorMessage.Get().GetValue());
	}

	if (Tool->HasCachedKeysToEase())
	{
		PaintSelectionText(ScaleInfo, InGeometry, OutDrawElements, ++InLayerId, DrawEffects);
	}

	if (DragState == EDragState::MarqueeSelect)
	{
		const FVector2D MarqueTopLeft(FMath::Min(MouseDownLocation.X, MouseMoveLocation.X), FMath::Min(MouseDownLocation.Y, MouseMoveLocation.Y));
		const FVector2D MarqueBottomRight(FMath::Max(MouseDownLocation.X, MouseMoveLocation.X), FMath::Max(MouseDownLocation.Y, MouseMoveLocation.Y));

		FSlateDrawElement::MakeBox(OutDrawElements, ++InLayerId
			, InGeometry.ToPaintGeometry(MarqueBottomRight - MarqueTopLeft, FSlateLayoutTransform(MarqueTopLeft))
			, FAppStyle::GetBrush(TEXT("MarqueeSelection")));
	}

	return InLayerId;
}

int32 SEaseCurveEditor::PaintGrid(const FTrackScaleInfo& InScaleInfo
	, const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const FSlateRect& InMyCullingRect
	, const ESlateDrawEffect InDrawEffects) const
{
	if (!GridSize.IsSet())
	{
		return InLayerId;
	}

	const float GridStep = 1.f / GridSize.Get();

	// Outside normalized area grid lines
	{
		const FVector2D LocalSize = InGeometry.GetLocalSize();
		const float ScreenGridStepX = FMath::Abs(InScaleInfo.InputToLocalX(GridStep) - InScaleInfo.InputToLocalX(0.f));
		const float ScreenGridStepY = FMath::Abs(InScaleInfo.OutputToLocalY(GridStep) - InScaleInfo.OutputToLocalY(0.f));

		// Vertical grid lines
		const float StartX = -FMath::Fractional(InScaleInfo.LocalXToInput(0.f) / GridStep) * ScreenGridStepX;

		if (ScreenGridStepX >= 1.f)
		{
			for (float ScreenX = StartX; ScreenX < LocalSize.X; ScreenX += ScreenGridStepX)
			{
				TArray<FVector2D> LinePoints;
				LinePoints.Add(FVector2D(ScreenX, 0.f));
				LinePoints.Add(FVector2D(ScreenX, LocalSize.Y));

				FSlateDrawElement::MakeLines(OutDrawElements, InLayerId, InGeometry.ToPaintGeometry(), LinePoints, InDrawEffects, ExtendedGridColor, false);
			}
		}

		// Horizontal grid lines
		const float StartY = FMath::Fractional(InScaleInfo.LocalYToOutput(0.f) / GridStep) * ScreenGridStepY;

		if (ScreenGridStepY >= 1.f)
		{
			for (float ScreenY = StartY; ScreenY < LocalSize.Y; ScreenY += ScreenGridStepY)
			{
				TArray<FVector2D> LinePoints;
				LinePoints.Add(FVector2D(0.f, ScreenY));
				LinePoints.Add(FVector2D(LocalSize.X, ScreenY));

				FSlateDrawElement::MakeLines(OutDrawElements, InLayerId
					, InGeometry.ToPaintGeometry(), LinePoints, InDrawEffects, ExtendedGridColor, false);
			}
		}
	}

	// Normalized area grid
	const int32 NormalizedLayerId = InLayerId + 1;
	{
		// Vertical grid lines
		for (float X = 0.f; X < 1.f; X += GridStep)
		{
			const float ScreenX = InScaleInfo.InputToLocalX(X);

			const float CurveZeroY1 = InScaleInfo.OutputToLocalY(0.f);
			const float CurveZeroY2 = InScaleInfo.OutputToLocalY(1.f);

			TArray<FVector2D> LinePoints;
			LinePoints.Add(FVector2D(ScreenX, CurveZeroY1));
			LinePoints.Add(FVector2D(ScreenX, CurveZeroY2));

			FSlateDrawElement::MakeLines(OutDrawElements, NormalizedLayerId
				, InGeometry.ToPaintGeometry(), LinePoints, InDrawEffects, GridColor, false);
		}

		// Horizontal grid lines
		for (float Y = 0.f; Y < 1.f; Y += GridStep)
		{
			const float ScreenY = InScaleInfo.OutputToLocalY(Y);

			const float CurveZeroX1 = InScaleInfo.InputToLocalX(0.f);
			const float CurveZeroX2 = InScaleInfo.InputToLocalX(1.f);

			TArray<FVector2D> LinePoints;
			LinePoints.Add(FVector2D(CurveZeroX1, ScreenY));
			LinePoints.Add(FVector2D(CurveZeroX2, ScreenY));

			FSlateDrawElement::MakeLines(OutDrawElements, NormalizedLayerId
				, InGeometry.ToPaintGeometry(), LinePoints, InDrawEffects, GridColor, false);
		}
	}

	return NormalizedLayerId;
}

void SEaseCurveEditor::PaintNormalBounds(const FTrackScaleInfo& InScaleInfo
	, const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const FSlateRect& InMyCullingRect
	, const ESlateDrawEffect InDrawEffects
	, const FWidgetStyle& InWidgetStyle) const
{
	const float ZeroInputX = InScaleInfo.InputToLocalX(0.f);
	const float OneInputX = InScaleInfo.InputToLocalX(1.f);
	const float ZeroOutputY = InScaleInfo.OutputToLocalY(0.f);
	const float OneOutputY = InScaleInfo.OutputToLocalY(1.f);

	// Time = 0 line
	TArray<FVector2D> ZeroTimeLinePoints;
	ZeroTimeLinePoints.Add(FVector2D(ZeroInputX, 0.f));
	ZeroTimeLinePoints.Add(FVector2D(ZeroInputX, InGeometry.GetLocalSize().Y));
	FSlateDrawElement::MakeLines(OutDrawElements, InLayerId, InGeometry.ToPaintGeometry()
		, ZeroTimeLinePoints, InDrawEffects, NormalBoundsColor, false, NormalBoundsThickness);

	// Time = 1 line
	TArray<FVector2D> OneTimeLinePoints;
	OneTimeLinePoints.Add(FVector2D(OneInputX, 0.f));
	OneTimeLinePoints.Add(FVector2D(OneInputX, InGeometry.GetLocalSize().Y));
	FSlateDrawElement::MakeLines(OutDrawElements, InLayerId, InGeometry.ToPaintGeometry()
		, OneTimeLinePoints, InDrawEffects, NormalBoundsColor, false, NormalBoundsThickness);

	// Value = 0 line
	TArray<FVector2D> ZeroValueLinePoints;
	ZeroValueLinePoints.Add(FVector2D(0.f, ZeroOutputY));
	ZeroValueLinePoints.Add(FVector2D(InGeometry.GetLocalSize().X, ZeroOutputY));
	FSlateDrawElement::MakeLines(OutDrawElements, InLayerId, InGeometry.ToPaintGeometry()
		, ZeroValueLinePoints, InDrawEffects, NormalBoundsColor, false, NormalBoundsThickness);

	// Value = 1 line
	TArray<FVector2D> OneValueLinePoints;
	OneValueLinePoints.Add(FVector2D(0.f, OneOutputY));
	OneValueLinePoints.Add(FVector2D(InGeometry.GetLocalSize().X, OneOutputY));
	FSlateDrawElement::MakeLines(OutDrawElements, InLayerId, InGeometry.ToPaintGeometry()
		, OneValueLinePoints, InDrawEffects, NormalBoundsColor, false, NormalBoundsThickness);
}

void SEaseCurveEditor::PaintCurve(const FTrackScaleInfo& InScaleInfo
	, const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 LayerId
	, const FSlateRect& MyCullingRect
	, const ESlateDrawEffect DrawEffects
	, const FWidgetStyle& InWidgetStyle) const
{
	const float NumKeys = EaseCurve->FloatCurve.GetNumKeys();
	if (NumKeys != 2)
	{
		return;
	}

	// Gather key handles and time/value pairs
	TArray<FKeyHandle> KeyHandles;
	TArray<TPair<float, float>> Key_TimeValuePairs;
	
	KeyHandles.Reserve(NumKeys);
	Key_TimeValuePairs.Reserve(NumKeys);

	for (auto It = EaseCurve->FloatCurve.GetKeyHandleIterator(); It; ++It)
	{
		const FKeyHandle& KeyHandle = *It;

		KeyHandles.Add(KeyHandle);
		Key_TimeValuePairs.Emplace(EaseCurve->FloatCurve.GetKeyTimeValuePair(KeyHandle));
	}

	// Draw curve lines
	TArray<FVector2D> LinePoints;
	TArray<FLinearColor> LineColors;

	for (int32 Index = 0; Index < NumKeys - 1; ++Index)
	{
		CreateLinesForSegment(EaseCurve->FloatCurve.GetKeyInterpMode(KeyHandles[Index]), EaseCurve->FloatCurve.GetKeyTangentMode(KeyHandles[Index])
			, Key_TimeValuePairs[Index], Key_TimeValuePairs[Index + 1], LinePoints, LineColors, InScaleInfo);

		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, InGeometry.ToPaintGeometry()
			, LinePoints, LineColors, DrawEffects, FLinearColor::White/*Color*/, true, CurveThickness * InGeometry.Scale);

		LinePoints.Reset();
		LineColors.Reset();
	}
}

void SEaseCurveEditor::CreateLinesForSegment(const ERichCurveInterpMode InInterpMode
	, const ERichCurveTangentMode InTangentMode
	, const TPair<float, float>& InStartKeyTimeValue, const TPair<float, float>& InEndKeyTimeValue
	, TArray<FVector2D>& OutLinePoints, TArray<FLinearColor>& OutLineColors
	, const FTrackScaleInfo& InScaleInfo) const
{
	static constexpr FLinearColor DisabledFadeFactor(0.03f, 0.03f, 0.03f, 1.f);

	const bool bIsEaseCurve = CanEditCurve();

	FLinearColor FadedCurveColor = CurveColor;
	FadedCurveColor.A = 0.05f;

	auto AddSegment = [this, &OutLinePoints, &OutLineColors, &bIsEaseCurve](const FVector2D& InPoint, FLinearColor InColor)
	{
		OutLinePoints.Add(InPoint);
		OutLineColors.Add(bIsEaseCurve ? MoveTemp(InColor) : (CurveColor * DisabledFadeFactor));
	};

	switch (InInterpMode)
	{
	case RCIM_Constant:
	{
		//@todo: should really only need 3 points here but something about the line rendering isn't quite behaving as I'd expect, so need extras
		AddSegment(FVector2D(InStartKeyTimeValue.Key, InStartKeyTimeValue.Value), CurveColor);
		AddSegment(FVector2D(InEndKeyTimeValue.Key, InStartKeyTimeValue.Value), CurveColor);
		AddSegment(FVector2D(InEndKeyTimeValue.Key, InStartKeyTimeValue.Value), CurveColor);
		AddSegment(FVector2D(InEndKeyTimeValue.Key, InEndKeyTimeValue.Value), CurveColor);
		AddSegment(FVector2D(InEndKeyTimeValue.Key, InStartKeyTimeValue.Value), CurveColor);

		break;
	}

	case RCIM_Linear:
	{
		AddSegment(FVector2D(InStartKeyTimeValue.Key, InStartKeyTimeValue.Value), CurveColor);
		AddSegment(FVector2D(InEndKeyTimeValue.Key, InEndKeyTimeValue.Value), CurveColor);

		break;
	}

	case RCIM_Cubic:
	{
		// Clamp to screen to avoid massive slowdown when zoomed in
		const float StartX = FMath::Max(InScaleInfo.InputToLocalX(InStartKeyTimeValue.Key), 0.0f);
		const float EndX = FMath::Min(InScaleInfo.InputToLocalX(InEndKeyTimeValue.Key), InScaleInfo.WidgetSize.X);

		constexpr float StepSize = 1.0f;
		const float CurveLengthX = EndX - StartX;

		for (float CurrentX = StartX; CurrentX < EndX; CurrentX += StepSize)
		{
			// Get color
			FLinearColor Color = CurveColor;
			switch (Operation.Get(EEaseCurveToolOperation::InOut))
			{
			case EEaseCurveToolOperation::Out:
				{
					const float Alpha = CurrentX / CurveLengthX;
					Color = FLinearColor::LerpUsingHSV(FadedCurveColor, CurveColor,  1.f - Alpha);
					break;
				}
			case EEaseCurveToolOperation::In:
				{
					const float Alpha = CurrentX / CurveLengthX;
					Color = FLinearColor::LerpUsingHSV(FadedCurveColor, CurveColor, Alpha);
					break;
				}
			case EEaseCurveToolOperation::InOut:
			default:
				{
					Color = CurveColor;
					break;
				}
			}
			
			// Add segment
			const float CurveIn = InScaleInfo.LocalXToInput(FMath::Min(CurrentX, EndX));
			const float CurveOut = EaseCurve->FloatCurve.Eval(CurveIn);
			AddSegment(FVector2D(CurveIn, CurveOut), Color);
		}

		AddSegment(FVector2D(InEndKeyTimeValue.Key, InEndKeyTimeValue.Value), CurveColor);

		break;
	}
	default:
		break;
	}

	// Transform to screen
	for (auto It = OutLinePoints.CreateIterator(); It; ++It)
	{
		FVector2D Vec2D = *It;
		Vec2D.X = InScaleInfo.InputToLocalX(Vec2D.X);
		Vec2D.Y = InScaleInfo.OutputToLocalY(Vec2D.Y);
		*It = Vec2D;
	}
}

int32 SEaseCurveEditor::PaintKeys(const FTrackScaleInfo& InScaleInfo
	, const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const ESlateDrawEffect InDrawEffects
	, const FWidgetStyle& InWidgetStyle) const
{
	if (EaseCurve->FloatCurve.GetNumKeys() < 2)
	{
		return InLayerId;
	}

	const int32 LayerId = InLayerId;
	const int32 SelectedLayerId = InLayerId + 1;

	// Iterate over each key
	ERichCurveInterpMode LastInterpMode = RCIM_Linear;
	for (auto It(EaseCurve->FloatCurve.GetKeyHandleIterator()); It; ++It)
	{
		FKeyHandle KeyHandle = *It;

		const FVector2D KeyLocation(InScaleInfo.InputToLocalX(EaseCurve->FloatCurve.GetKeyTime(KeyHandle))
			, InScaleInfo.OutputToLocalY(EaseCurve->FloatCurve.GetKeyValue(KeyHandle)));
		const FVector2D KeyIconLocation = KeyLocation - HalfKeyHitSize;
		const bool bIsSelected = (KeyHandle == SelectedTangent.KeyHandle);
		const FSlateBrush* KeyBrush = FAppStyle::GetBrush("CurveEd.CurveKey");
		const int32 LayerToUse = bIsSelected ? SelectedLayerId : LayerId;

		FSlateDrawElement::MakeBox(OutDrawElements, LayerToUse,
			InGeometry.ToPaintGeometry(KeyHitSize, FSlateLayoutTransform(KeyIconLocation)),
			KeyBrush, InDrawEffects,
			KeyBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint());

		// Draw the tangent handles
		const int32 KeyIndex = EaseCurve->FloatCurve.GetIndexSafe(KeyHandle);

		if (EaseCurve->FloatCurve.GetKeyInterpMode(KeyHandle) == RCIM_Cubic || LastInterpMode == RCIM_Cubic)
		{
			if (KeyIndex == 0)
			{
				PaintTangentHandle(InGeometry, OutDrawElements, LayerToUse, InDrawEffects, InWidgetStyle
					, KeyLocation, GetLeaveTangentScreenLocation(InScaleInfo, KeyHandle), bIsSelected);
			}
			else if (KeyIndex == 1)
			{
				PaintTangentHandle(InGeometry, OutDrawElements, LayerToUse, InDrawEffects, InWidgetStyle
					, KeyLocation, GetArriveTangentScreenLocation(InScaleInfo, KeyHandle), bIsSelected);
			}
		}
		else
		{
			PaintTangentHandle(InGeometry, OutDrawElements, LayerToUse, InDrawEffects, InWidgetStyle
				, KeyLocation, GetLeaveTangentScreenLocation(InScaleInfo, KeyHandle), bIsSelected);

			PaintTangentHandle(InGeometry, OutDrawElements, LayerToUse, InDrawEffects, InWidgetStyle
				, KeyLocation, GetArriveTangentScreenLocation(InScaleInfo, KeyHandle), bIsSelected);
		}

		LastInterpMode = EaseCurve->FloatCurve.GetKeyInterpMode(KeyHandle);
	}

	return SelectedLayerId;
}

int32 SEaseCurveEditor::PaintTangentHandle(const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const ESlateDrawEffect InDrawEffects
	, const FWidgetStyle& InWidgetStyle
	, const FVector2D& InKeyLocation
	, const FVector2D& InTangentLocation
	, const bool bInSelected) const
{
	// Draw line to tangent control handle
	{
		const FLinearColor LineColor = FAppStyle::GetColor("CurveEd.TangentColor");

		TArray<FVector2D> LinePoints;
		LinePoints.Add(FVector2D(InKeyLocation));
		LinePoints.Add(FVector2D(InTangentLocation));

		FSlateDrawElement::MakeLines(OutDrawElements, InLayerId++, InGeometry.ToPaintGeometry()
			, LinePoints, InDrawEffects, LineColor, true, TangentHandleLineThickness);
	}

	// Draw tangent handle
	{
		const FSlateBrush* ImageBrush = FAppStyle::GetBrush(TEXT("Icons.BulletPoint"));
		const FVector2D TangentIconLocation = InTangentLocation - HalfTangentDrawSize;
		const FLinearColor HandleColor = bInSelected ? FStyleColors::AccentBlue.GetSpecifiedColor() : FLinearColor::White;

		FSlateDrawElement::MakeBox(OutDrawElements, InLayerId++
			, InGeometry.ToPaintGeometry(TangentDrawSize, FSlateLayoutTransform(TangentIconLocation))
			, ImageBrush, InDrawEffects, HandleColor * InWidgetStyle.GetColorAndOpacityTint());
	}

	return InLayerId;
}

int32 SEaseCurveEditor::PaintSelectionText(const FTrackScaleInfo& InScaleInfo
	, const FGeometry& InGeometry
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const ESlateDrawEffect InDrawEffects) const
{
	const FSlateFontInfo FontInfo = FEaseCurveStyle::Get().GetFontStyle(TEXT("Editor.ErrorFont"));
	const FText SelectionText = Tool->IsCurveEditorSelection()
		? LOCTEXT("CurveEditorSelection", "Curve Editor Selection")
		: LOCTEXT("SequencerSelection", "Sequencer Selection");

	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FVector2D TextSize = FontMeasure->Measure(SelectionText, FontInfo);

	const float HalfInputX = InScaleInfo.InputToLocalX(0.5f);
	const float FullOutputY = InScaleInfo.OutputToLocalY(1.0f);

	const FVector2D HalfTextSize = TextSize * 0.5f;
	const FVector2D ActualOffset(HalfInputX - HalfTextSize.X, FullOutputY - TextSize.Y);

	static const FLinearColor TintColor = FStyleColors::Foreground.GetSpecifiedColor().CopyWithNewOpacity(0.5f);

	FSlateDrawElement::MakeText(OutDrawElements, ++InLayerId
		, InGeometry.ToPaintGeometry(InGeometry.Size, FSlateLayoutTransform(ActualOffset))
		, SelectionText, FontInfo, InDrawEffects, TintColor);

	return InLayerId;
}

void SEaseCurveEditor::SetRequireFocusToZoom(const bool bInRequireFocusToZoom)
{
	bRequireFocusToZoom = bInRequireFocusToZoom;
}

TOptional<bool> SEaseCurveEditor::OnQueryShowFocus(const EFocusCause InFocusCause) const
{
	if (bRequireFocusToZoom)
	{
		// Enable showing a focus rectangle when the widget has keyboard focus
		return TOptional<bool>(true);
	}
	return TOptional<bool>();
}

FReply SEaseCurveEditor::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FKey EffectingButton = InMouseEvent.GetEffectingButton();

	DragState = EDragState::PreDrag;

	if (EffectingButton == EKeys::LeftMouseButton)
	{
		MouseDownLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

		if (!HasKeyboardFocus())
		{
			FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (EffectingButton == EKeys::RightMouseButton)
	{
		if (HitTestCurves(InMyGeometry, InMouseEvent))
		{
			FPlatformApplicationMisc::ClipboardCopy(*EaseCurve->GetTangents().ToJson());

			FEaseCurveTool::ShowNotificationMessage(LOCTEXT("EaseCurveToolTangentsCopied", "Ease Curve Tool Tangents Copied!"));
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

void SEaseCurveEditor::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (DragState == EDragState::DragTangent)
	{
		OnDragEnd.ExecuteIfBound();
	}
	
	DragState = EDragState::None;
}

FReply SEaseCurveEditor::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (this->HasMouseCapture())
	{
		if (DragState == EDragState::PreDrag)
		{
			ProcessClick(InMyGeometry, InMouseEvent);
		}
		else
		{
			EndDrag(InMyGeometry, InMouseEvent);
		}

		ClearSelection();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void ClampViewRangeToDataIfBound(float& NewViewMin, float& NewViewMax
	, const TAttribute<TOptional<float>>& DataMin, const TAttribute<TOptional<float>>& DataMax, const float ViewRange)
{
	const TOptional<float>& Min = DataMin.Get();
	const TOptional<float>& Max = DataMax.Get();
	if (Min.IsSet() && NewViewMin < Min.GetValue())
	{
		NewViewMin = Min.GetValue();
		NewViewMax = ViewRange;
	}
	else if (Max.IsSet() && NewViewMax > Max.GetValue())
	{
		NewViewMin = Max.GetValue() - ViewRange;
		NewViewMax = Max.GetValue();
	}
}

FReply SEaseCurveEditor::OnMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	UpdateCurveToolTip(InMyGeometry, InMouseEvent);

	if (this->HasMouseCapture())
	{
		if (DragState == EDragState::PreDrag)
		{
			TryStartDrag(InMyGeometry, InMouseEvent);
		}
		if (DragState != EDragState::None)
		{
			ProcessDrag(InMyGeometry, InMouseEvent);
		}

		MouseMoveLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<IToolTip> SEaseCurveEditor::CreateCurveToolTip()
{
	const FEaseCurveTangents Tangents = EaseCurve->GetTangents();

	SEaseCurvePreview::FArguments PreviewArgs;
	PreviewArgs.Tangents(Tangents)
		.PreviewSize(256.f)
		.CanExpandPreview(true)
		.CustomToolTip(false)
		.Animate(true)
		.DisplayRate(DisplayRate.Get())
		.DrawMotionTrails(true);

	const TSharedRef<SWidget> AdditionalContent = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 5.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.Text(this, &SEaseCurveEditor::GetCurveToolTipInputText)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 5.f, 0.f, 15.f)
		[
			SNew(STextBlock)
			.Text(this, &SEaseCurveEditor::GetCurveToolTipOutputText)
		];

	return SNew(SToolTip)
		.TextMargin(0.f)
		.Visibility_Lambda([this]()
			{
				return (ToolTipIndex != INDEX_NONE) ? EVisibility::Visible : EVisibility::Hidden;
			})
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]()
				{
					return (ToolTipIndex == INDEX_NONE) ? 0 : ToolTipIndex;
				})
			+ SWidgetSwitcher::Slot()
			[
				SEaseCurvePreviewToolTip::CreateDefaultToolTip(PreviewArgs, AdditionalContent)
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Margin(FMargin(5.f))
				.Text(StartTooltipText)
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Margin(FMargin(5.f))
				.Text(EndTooltipText)
			]
		];
}

void SEaseCurveEditor::UpdateCurveToolTip(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InMyGeometry.GetLocalSize());
	const FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	const float Time = ScaleInfo.LocalXToInput(MousePosition.X);
	const float Value = EaseCurve->FloatCurve.Eval(Time);
	const FVector2D TimeValueRange = FVector2D(Time, Value);

	FNumberFormattingOptions FormattingOptions;
	FormattingOptions.MaximumFractionalDigits = 2;
	CurveToolTipOutputText = FText::Format(LOCTEXT("CurveToolTipValueFormat", "Value:\t{0}"), FText::AsNumber(TimeValueRange.Y, &FormattingOptions));
	CurveToolTipInputText = FText::Format(LOCTEXT("CurveToolTipTimeFormat", "Time:\t{0}"), FText::AsNumber(TimeValueRange.X, &FormattingOptions));

	if (DragState != EDragState::None
		|| HasDrawErrorMessage()
		|| !CanEditCurve())
	{
		ToolTipIndex = INDEX_NONE;
		CurveToolTip.Reset();
		SetToolTip(CurveToolTip);
		return;
	}

	if (StartTooltipText.IsSet() && HitTestKey(InMyGeometry, InMouseEvent, FVector2D(0.f, 0.f)))
	{
		ToolTipIndex = 1;
	}
	else if (EndTooltipText.IsSet() && HitTestKey(InMyGeometry, InMouseEvent, FVector2D(1.f, 1.f)))
	{
		ToolTipIndex = 2;
	}
	else if (HitTestCurves(InMyGeometry, InMouseEvent))
	{
		ToolTipIndex = 0;
	}
	else
	{
		ToolTipIndex = INDEX_NONE;
	}

	if (ToolTipIndex == INDEX_NONE)
	{
		CurveToolTip.Reset();
		SetToolTip(CurveToolTip);
		return;
	}

	if (!CurveToolTip.IsValid())
	{
		CurveToolTip = CreateCurveToolTip();
		SetToolTip(CurveToolTip);
	}
}

FReply SEaseCurveEditor::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bRequireFocusToZoom || HasKeyboardFocus())
	{
		ZoomView(FVector2D(MouseEvent.GetWheelDelta(), MouseEvent.GetWheelDelta()));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SEaseCurveEditor::ZoomView(const FVector2D& InDelta)
{
	const FVector2D ZoomDelta = -0.1f * InDelta;

	const float OutputViewSize = ViewMaxOutput - ViewMinOutput;
	const float OutputChange = OutputViewSize * ZoomDelta.Y;
	const float HalfOutputChange = OutputChange * 0.5f;

	const float NewMinOutput = ViewMinOutput - HalfOutputChange;
	const float NewMaxOutput = ViewMaxOutput + HalfOutputChange;

	SetOutputMinMax(NewMinOutput, NewMaxOutput);

	const float InputViewSize = ViewMaxInput - ViewMinInput;
	const float InputChange = InputViewSize * ZoomDelta.X;
	const float HalfInputChange = InputChange * 0.5f;

	const float NewMinInput = ViewMinInput - HalfInputChange;
	const float NewMaxInput = ViewMaxInput + HalfInputChange;

	SetInputMinMax(NewMinInput, NewMaxInput);
}

FReply SEaseCurveEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FModifierKeysState& ModifierKeys = InKeyEvent.GetModifierKeys();

	FReply Reply = FReply::Unhandled();

	if (ModifierKeys.IsAltDown()) // lock tangent
	{
		LockTangents = MovingTangents;
		Reply = FReply::Handled();
	}
	else if (ModifierKeys.IsControlDown()) // lock weight
	{
		LockTangents = MovingTangents;
		Reply = FReply::Handled();
	}
	else if (ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown()) // mirror tangents and weights
	{
		LockTangents = MovingTangents;
		Reply = FReply::Handled();
	}

	if (OnKeyDownEvent.IsBound())
	{
		Reply = OnKeyDownEvent.Execute(MyGeometry, InKeyEvent);
	}

	return Reply;
}

void SEaseCurveEditor::TryStartDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!CanEditCurve())
	{
		return;
	}

	const bool bLeftMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bMiddleMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
	const bool bRightMouseButton = InMouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);

	const FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	const FVector2D DragVector = MousePosition - MouseDownLocation;

	if (DragVector.SizeSquared() >= FMath::Square(DragThreshold))
	{
		if (bLeftMouseButton)
		{
			// Check if we should start dragging a tangent.
			const FSelectedTangent HitTangent = HitTestTangentHandles(InMyGeometry, InMouseEvent);
			if (HitTangent.IsValid())
			{
				SelectedTangent = HitTangent;

				OnDragStart.ExecuteIfBound();

				DragState = EDragState::DragTangent;

				PreDragTangents = EaseCurve->GetTangents();
			}
			else
			{
				// Otherwise if the user left clicked on nothing and start a marquee select.
				DragState = EDragState::MarqueeSelect;
			}
		}
		else if (bMiddleMouseButton)
		{
			DragState = EDragState::Pan;
		}
		else if (bRightMouseButton)
		{
			DragState = EDragState::Pan;
		}
		else
		{
			DragState = EDragState::None;
		}
	}
}

void SEaseCurveEditor::ProcessDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InMyGeometry.GetLocalSize());
	const FVector2D ScreenDelta = InMouseEvent.GetCursorDelta();

	FVector2D InputDelta;
	InputDelta.X = ScreenDelta.X / ScaleInfo.PixelsPerInput;
	InputDelta.Y = -ScreenDelta.Y / ScaleInfo.PixelsPerOutput;
	
	switch (DragState)
	{
	case EDragState::DragTangent:
	{
		const FVector2D MousePositionScreen = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		MoveSelectedTangent(ScaleInfo, MousePositionScreen - MouseDownLocation);
		break;
	}
	case EDragState::Pan:
	{
		// Output is not clamped.
		const float NewMinOutput = ViewMinOutput - InputDelta.Y;
		const float NewMaxOutput = ViewMaxOutput - InputDelta.Y;
		SetOutputMinMax(NewMinOutput, NewMaxOutput);

		// Input maybe clamped if DataMinInput or DataMaxOutput was set.
		float NewMinInput = ViewMinInput - InputDelta.X;
		float NewMaxInput = ViewMaxInput - InputDelta.X;
		ClampViewRangeToDataIfBound(NewMinInput, NewMaxInput, DataMinInput, DataMaxInput, ScaleInfo.ViewInputRange);
		SetInputMinMax(NewMinInput, NewMaxInput);
		break;
	}
	case EDragState::Zoom:
	{
		const FVector2D Delta = FVector2D(ScreenDelta.X * 0.05f, ScreenDelta.X * 0.05f);
		ZoomView(Delta);
		break;
	}
	default:
		break;
	}
}

void SEaseCurveEditor::EndDrag(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (DragState == EDragState::DragTangent)
	{
		OnDragEnd.ExecuteIfBound();
	}
	else if (DragState == EDragState::MarqueeSelect)
	{
		if (!InMouseEvent.IsControlDown() && !InMouseEvent.IsShiftDown())
		{
			ClearSelection();
		}
	}

	DragState = EDragState::None;
}

void SEaseCurveEditor::ProcessClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FKey EffectingButton = InMouseEvent.GetEffectingButton();

	if (EffectingButton == EKeys::LeftMouseButton)
	{
		const FSelectedTangent HitTangent = HitTestTangentHandles(InMyGeometry, InMouseEvent);
		if (HitTangent.IsValid())
		{
			SelectedTangent = HitTangent;
		}
	}
	else if (EffectingButton == EKeys::RightMouseButton)
	{
		if (GetContextMenuContent.IsBound() && !HitTestCurves(InMyGeometry, InMouseEvent))
		{
			const FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, GetContextMenuContent.Execute()
				, FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}
	}
}

void SEaseCurveEditor::ClearSelection()
{
	SelectedTangent = FSelectedTangent();
}

void SEaseCurveEditor::SetDefaultOutput(const float InMinZoomRange)
{
	const float HalfMinZoomRange = InMinZoomRange * 0.5f;
	const float NewMinOutput = ViewMinOutput - HalfMinZoomRange;
	const float NewMaxOutput = ViewMaxOutput + HalfMinZoomRange;

	SetOutputMinMax(NewMinOutput, NewMaxOutput);
}

void SEaseCurveEditor::ZoomToFit()
{
	const FRichCurveKey& StartKey = EaseCurve->GetStartKey();
	const FRichCurveKey& EndKey = EaseCurve->GetEndKey();

	const FVector2D ArriveTangentDir = CalcTangentDir(EndKey.ArriveTangent) * EndKey.ArriveTangentWeight;
	const FVector2D LeaveTangentDir = CalcTangentDir(StartKey.LeaveTangent) * StartKey.LeaveTangentWeight;

	const FVector2D StartKeyPosition(EaseCurve->FloatCurve.GetKeyTime(EaseCurve->GetStartKeyHandle())
		, EaseCurve->FloatCurve.GetKeyValue(EaseCurve->GetStartKeyHandle()));
	const FVector2D EndKeyPosition(EaseCurve->FloatCurve.GetKeyTime(EaseCurve->GetEndKeyHandle())
		, EaseCurve->FloatCurve.GetKeyValue(EaseCurve->GetEndKeyHandle()));
	const FVector2D ArrivePosition = -ArriveTangentDir + EndKeyPosition;
	const FVector2D LeavePosition = LeaveTangentDir - StartKeyPosition;

	// Fit horizontally
	{
		float InMinX = FLT_MAX;
		float InMaxX = -FLT_MAX;

		float MinTimeX = 0.f;
		float MaxTimeX = 0.f;
		EaseCurve->FloatCurve.GetTimeRange(MinTimeX, MaxTimeX);

		InMinX = FMath::Min(MinTimeX, InMinX);
		InMaxX = FMath::Max(MaxTimeX, InMaxX);

		InMinX = FMath::Min(LeavePosition.X, InMinX);
		InMaxX = FMath::Max(LeavePosition.X, InMaxX);

		InMinX = FMath::Min(ArrivePosition.X, InMinX);
		InMaxX = FMath::Max(ArrivePosition.X, InMaxX);

		// Clamp the minimum size
		float SizeX = InMaxX - InMinX;
		if (SizeX < MinViewRange)
		{
			InMinX -= HalfMinViewRange;
			InMaxX += HalfMinViewRange;
			SizeX = InMaxX - InMinX;
		}

		// Add margin
		InMinX -= FitMargin * SizeX;
		InMaxX += FitMargin * SizeX;

		SetInputMinMax(InMinX, InMaxX);
	}

	// Fit vertically
	{
		float InMinY = FLT_MAX;
		float InMaxY = -FLT_MAX;

		float MinValY = 0.f;
		float MaxValY = 0.f;
		EaseCurve->FloatCurve.GetValueRange(MinValY, MaxValY);

		InMinY = FMath::Min(MinValY, InMinY);
		InMaxY = FMath::Max(MaxValY, InMaxY);

		InMinY = FMath::Min(LeavePosition.Y, InMinY);
		InMaxY = FMath::Max(LeavePosition.Y, InMaxY);

		InMinY = FMath::Min(ArrivePosition.Y, InMinY);
		InMaxY = FMath::Max(ArrivePosition.Y, InMaxY);

		// If in max and in min is same, then include 0.f
		if (InMaxY == InMinY)
		{
			InMaxY = FMath::Max(InMaxY, 0.f);
			InMinY = FMath::Min(InMinY, 0.f);
		}

		// Clamp the minimum size
		float SizeY = InMaxY - InMinY;
		if (SizeY < MinViewRange)
		{
			SetDefaultOutput(MinViewRange);

			InMinY = ViewMinOutput;
			InMaxY = ViewMaxOutput;
			SizeY = InMaxY - InMinY;
		}

		// Add margin
		const float NewMinOutputY = (InMinY - FitMargin * SizeY);
		const float NewMaxOutputY = (InMaxY + FitMargin * SizeY);

		SetOutputMinMax(NewMinOutputY, NewMaxOutputY);
	}
}

FString SEaseCurveEditor::GetReferencerName() const
{
	return TEXT("SEaseCurveEditor");
}

void SEaseCurveEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EaseCurve);
}

void SEaseCurveEditor::SetInputMinMax(const float InNewMin, const float InNewMax)
{
	ViewMinInput = InNewMin;
	ViewMaxInput = InNewMax;
}

void SEaseCurveEditor::SetOutputMinMax(const float InNewMin, const float InNewMax)
{
	ViewMinOutput = InNewMin;
	ViewMaxOutput = InNewMax;
}

bool SEaseCurveEditor::HitTestCurves(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) const
{
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InMyGeometry.GetLocalSize());
	const FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	const float Time = ScaleInfo.LocalXToInput(MousePosition.X);
	const float Value = ScaleInfo.OutputToLocalY(EaseCurve->FloatCurve.Eval(Time));

	if (MousePosition.Y > (Value - HalfCurveHitSize.Y)
		&& MousePosition.Y < (Value + HalfCurveHitSize.Y))
	{
		return true;
	}

	return false;
}

SEaseCurveEditor::FSelectedTangent SEaseCurveEditor::HitTestTangentHandles(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) const
{
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InMyGeometry.GetLocalSize());
	const FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	auto IsInside = [&MousePosition](const FVector2D& InPosition) -> bool
		{
			return MousePosition.Y > (InPosition.Y - HalfTangentHitSize.Y)
				&& MousePosition.Y < (InPosition.Y + HalfTangentHitSize.Y)
				&& MousePosition.X > (InPosition.X - HalfTangentHitSize.X)
				&& MousePosition.X < (InPosition.X + HalfTangentHitSize.X);
		};

	FSelectedTangent OutTangent;

	const FKeyHandle StartKeyHandle = EaseCurve->FloatCurve.GetFirstKeyHandle();
	check(StartKeyHandle != FKeyHandle::Invalid());
	const FKeyHandle EndKeyHandle = EaseCurve->FloatCurve.GetLastKeyHandle();
	check(EndKeyHandle != FKeyHandle::Invalid());

	const FVector2D StartTangentScreenLocation = GetLeaveTangentScreenLocation(ScaleInfo, StartKeyHandle);
	const FVector2D EndTangentScreenLocation = GetArriveTangentScreenLocation(ScaleInfo, EndKeyHandle);

	const FVector2D StartKeyPosition(EaseCurve->FloatCurve.GetKeyTime(StartKeyHandle), EaseCurve->FloatCurve.GetKeyValue(StartKeyHandle));
	const FVector2D EndKeyPosition(EaseCurve->FloatCurve.GetKeyTime(EndKeyHandle), EaseCurve->FloatCurve.GetKeyValue(EndKeyHandle));

	const float StartDeltaPosition = (StartKeyPosition - StartTangentScreenLocation).Size();
	const float EndDeltaPosition = (EndKeyPosition - EndTangentScreenLocation).Size();

	const bool bIsInsideStartHandle = IsInside(StartTangentScreenLocation);
	const bool bIsInsideEndHandle = IsInside(EndTangentScreenLocation);

	if (bIsInsideStartHandle && bIsInsideEndHandle)
	{
		if (StartDeltaPosition < EndDeltaPosition)
		{
			OutTangent = FSelectedTangent(StartKeyHandle, false);
		}
		else if (EndDeltaPosition < StartDeltaPosition)
		{
			OutTangent = FSelectedTangent(EndKeyHandle, true);
		}
	}
	else if (bIsInsideStartHandle)
	{
		OutTangent = FSelectedTangent(StartKeyHandle, false);
	}
	else if (bIsInsideEndHandle)
	{
		OutTangent = FSelectedTangent(EndKeyHandle, true);
	}

	return OutTangent;
}

bool SEaseCurveEditor::HitTestKey(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent, const FVector2D& InInputPosition) const
{
	const FTrackScaleInfo ScaleInfo(ViewMinInput, ViewMaxInput, ViewMinOutput, ViewMaxOutput, InMyGeometry.GetLocalSize());
	const FVector2D MousePosition = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	const float Time = ScaleInfo.InputToLocalX(InInputPosition.X);
	const float Value = ScaleInfo.OutputToLocalY(InInputPosition.Y);

	return MousePosition.Y > (Value - HalfKeyHitSize.Y)
		&& MousePosition.Y < (Value + HalfKeyHitSize.Y)
		&& MousePosition.X > (Time - HalfKeyHitSize.X)
		&& MousePosition.X < (Time + HalfKeyHitSize.X);
}

void SEaseCurveEditor::MoveSelectedTangent(const FTrackScaleInfo& InScaleInfo, const FVector2D& InScreenDelta)
{
	if (!SelectedTangent.IsValid())
	{
		return;
	}

	const FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
	const bool bIsAltOnly = ModifierKeysState.IsAltDown() && !ModifierKeysState.IsControlDown() && !ModifierKeysState.IsShiftDown();
	const bool bIsControlOnly = !ModifierKeysState.IsAltDown() && ModifierKeysState.IsControlDown() && !ModifierKeysState.IsShiftDown();
	const bool bIsControlAlt = ModifierKeysState.IsAltDown() && ModifierKeysState.IsControlDown();

	const FVector2D SelectedKeyPosition(EaseCurve->FloatCurve.GetKeyTime(SelectedTangent.KeyHandle)
		, EaseCurve->FloatCurve.GetKeyValue(SelectedTangent.KeyHandle));

	const bool bSnapLocation = (bGridSnap.IsSet() && GridSize.IsSet())
		? (bGridSnap.Get() && !ModifierKeysState.IsShiftDown()) || (!bGridSnap.Get() && ModifierKeysState.IsShiftDown())
		: false;

	MovingTangents = PreDragTangents;
	FEaseCurveTangents NewTangents = MovingTangents;

	if (!SelectedTangent.bIsArrival) // Start / leave tangent
	{
		const FVector2D LeaveTangentScreenLocation = InScreenDelta + GetLeaveTangentScreenLocation(InScaleInfo
			, SelectedKeyPosition, PreDragTangents.Start, PreDragTangents.StartWeight);

		FVector2D NewLeaveDir(InScaleInfo.LocalXToInput(LeaveTangentScreenLocation.X)
			, InScaleInfo.LocalYToOutput(LeaveTangentScreenLocation.Y));
		NewLeaveDir.X = FMath::Clamp(NewLeaveDir.X, 0.f, 1.f);
		NewLeaveDir.Y = FMath::Clamp(NewLeaveDir.Y, -10.f, 10.f);
		if (bSnapLocation)
		{
			NewLeaveDir = SnapLocation(NewLeaveDir);
		}
		const FVector2D NewLeaveDirDelta = NewLeaveDir - SelectedKeyPosition;
		
		MovingTangents.Start = CalcTangent(NewLeaveDirDelta);
		MovingTangents.StartWeight = NewLeaveDirDelta.Size();

		if (bIsAltOnly) // lock tangent
		{
			NewTangents.Start = LockTangents.Start;
			NewTangents.StartWeight = MovingTangents.StartWeight;
		}
		else if (bIsControlOnly) // lock weight
		{
			NewTangents.Start = MovingTangents.Start;
			NewTangents.StartWeight = LockTangents.StartWeight;
		}
		else
		{
			NewTangents.Start = MovingTangents.Start;
			NewTangents.StartWeight = MovingTangents.StartWeight;

			if (bIsControlAlt) // mirror tangents and weights
			{
				NewTangents.End = NewTangents.Start;
				NewTangents.EndWeight = MovingTangents.StartWeight;
			}
		}
	}
	else // End / arrive tangent
	{
		const FVector2D ArriveTangentScreenLocation = InScreenDelta + GetArriveTangentScreenLocation(InScaleInfo
			, SelectedKeyPosition, PreDragTangents.End, PreDragTangents.EndWeight);

		FVector2D NewArriveDir(InScaleInfo.LocalXToInput(ArriveTangentScreenLocation.X)
			, InScaleInfo.LocalYToOutput(ArriveTangentScreenLocation.Y));
		NewArriveDir.X = FMath::Clamp(NewArriveDir.X, 0.f, 1.f);
		NewArriveDir.Y = FMath::Clamp(NewArriveDir.Y, -10.f, 10.f);
		if (bSnapLocation)
		{
			NewArriveDir = SnapLocation(NewArriveDir);
		}
		const FVector2D NewArriveDirDelta = NewArriveDir - SelectedKeyPosition;

		MovingTangents.End = CalcTangent(-1.f * NewArriveDirDelta);
		MovingTangents.EndWeight = NewArriveDirDelta.Size();

		if (bIsAltOnly) // lock tangent
		{
			NewTangents.End = LockTangents.End;
			NewTangents.EndWeight = MovingTangents.EndWeight;
		}
		else if (bIsControlOnly) // lock weight
		{
			NewTangents.End = MovingTangents.End;
			NewTangents.EndWeight = LockTangents.EndWeight;
		}
		else
		{
			NewTangents.End = MovingTangents.End;
			NewTangents.EndWeight = MovingTangents.EndWeight;

			if (bIsControlAlt) // mirror tangents and weights
			{
				NewTangents.Start = NewTangents.End;
				NewTangents.StartWeight = MovingTangents.EndWeight;
			}
		}
	}

	OnTangentsChanged.ExecuteIfBound(NewTangents);
}

FVector2d SEaseCurveEditor::GetArriveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo, const FKeyHandle& InKeyHandle) const
{
	const FVector2d KeyPosition(EaseCurve->FloatCurve.GetKeyTime(InKeyHandle), EaseCurve->FloatCurve.GetKeyValue(InKeyHandle));
	const FRichCurveKey& RichKey = EaseCurve->FloatCurve.GetKey(InKeyHandle);
	return GetArriveTangentScreenLocation(InScaleInfo, KeyPosition, RichKey.ArriveTangent, RichKey.ArriveTangentWeight);
}

FVector2d SEaseCurveEditor::GetArriveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo
	, const FVector2d& InKeyPosition, const float InTangent, const float InWeight) const
{
	// Get direction vector from tangent
	const FVector2d ArriveTangentDir = CalcTangentDir(InTangent);
	const FVector2d ArrivePosition = -ArriveTangentDir + InKeyPosition;

	FVector2d OutScreenArrivePosition = FVector2D(InScaleInfo.InputToLocalX(ArrivePosition.X), InScaleInfo.OutputToLocalY(ArrivePosition.Y));

	// Convert curve position to screen position
	const FVector2d KeyScreenPosition = FVector2D(InScaleInfo.InputToLocalX(InKeyPosition.X), InScaleInfo.OutputToLocalY(InKeyPosition.Y));

	const FVector2d DeltaScreenPosition = OutScreenArrivePosition - KeyScreenPosition;
	OutScreenArrivePosition = KeyScreenPosition + (DeltaScreenPosition * InWeight);

	return OutScreenArrivePosition;
}

FVector2d SEaseCurveEditor::GetLeaveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo, const FKeyHandle& InKeyHandle) const
{
	const FVector2d KeyPosition(EaseCurve->FloatCurve.GetKeyTime(InKeyHandle), EaseCurve->FloatCurve.GetKeyValue(InKeyHandle));
	const FRichCurveKey& RichKey = EaseCurve->FloatCurve.GetKey(InKeyHandle);
	return GetLeaveTangentScreenLocation(InScaleInfo, KeyPosition, RichKey.LeaveTangent, RichKey.LeaveTangentWeight);
}

FVector2D SEaseCurveEditor::GetLeaveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo
	, const FVector2d& InKeyPosition, const float InTangent, const float InWeight) const
{
	// Get direction vector from tangent
	const FVector2d LeaveTangentDir = CalcTangentDir(InTangent);
	const FVector2d LeavePosition = LeaveTangentDir + InKeyPosition;

	FVector2d OutScreenLeavePosition = FVector2D(InScaleInfo.InputToLocalX(LeavePosition.X), InScaleInfo.OutputToLocalY(LeavePosition.Y));
	
	// Convert curve position to screen position
	const FVector2d KeyScreenPosition = FVector2D(InScaleInfo.InputToLocalX(InKeyPosition.X), InScaleInfo.OutputToLocalY(InKeyPosition.Y));

	const FVector2d DeltaScreenPosition = OutScreenLeavePosition - KeyScreenPosition;
	OutScreenLeavePosition = KeyScreenPosition + (DeltaScreenPosition * InWeight);

	return OutScreenLeavePosition;
}

void SEaseCurveEditor::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (EaseCurve && EaseCurve->GetOwners().Contains(Object))
	{
		ClearSelection();
	}
}

void SEaseCurveEditor::PostUndo(const bool bInSuccess)
{
	ClearSelection();
}

void SEaseCurveEditor::PostRedo(const bool bInSuccess)
{
	PostUndo(bInSuccess);
}

FVector2D SEaseCurveEditor::SnapLocation(const FVector2D& InLocation) const
{
	if (!GridSize.IsSet())
	{
		return InLocation;
	}

	const double SnapGridSize = GridSize.Get();
	if (SnapGridSize == 0.f)
	{
		return InLocation;
	}

	const double SnapStep = 1.f / SnapGridSize;

	FVector2d OutLocation;
	OutLocation.X = FMath::RoundToInt(InLocation.X / SnapStep) * SnapStep;
	OutLocation.Y = FMath::RoundToInt(InLocation.Y / SnapStep) * SnapStep;
	
	return OutLocation;
}

bool SEaseCurveEditor::HasDrawErrorMessage() const
{
	return ErrorMessage.IsSet() && ErrorMessage.Get().IsSet();
}

bool SEaseCurveEditor::CanEditCurve() const
{
	return bCanEditCurve.Get(false);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
