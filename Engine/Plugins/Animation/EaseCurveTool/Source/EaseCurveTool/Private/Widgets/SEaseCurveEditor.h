// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Curves/KeyHandle.h"
#include "EaseCurveTangents.h"
#include "EaseCurveTool.h"
#include "EditorUndoClient.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Misc/Attribute.h"
#include "SCurveEditor.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IToolTip;

namespace UE::EaseCurveTool
{

DECLARE_DELEGATE_OneParam(FOnEaseCurveChanged, const FEaseCurveTangents& /*InTangents*/)

class SEaseCurveEditor : 
	public SCompoundWidget,
	public FGCObject,
	public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SEaseCurveEditor)
		: _DisplayRate(FFrameRate(30, 1))
		, _ShowInputGridNumbers(true)
		, _ShowOutputGridNumbers(true)
		, _NormalAreaColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.05f))
		, _NormalBoundsColor(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
		, _GridSnap(false)
		, _GridColor(FLinearColor(0.1f, 0.1f, 0.1f, 0.1f))
		, _ExtendedGridColor(FLinearColor(0.02f, 0.02f, 0.02f, 0.5f))
		, _CurveThickness(2.5f)
		, _CurveColor(FLinearColor::White)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
		SLATE_ATTRIBUTE(TOptional<float>, DataMinInput)
		SLATE_ATTRIBUTE(TOptional<float>, DataMaxInput)

		SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSize)
		SLATE_ATTRIBUTE(FFrameRate, DisplayRate)

		SLATE_ARGUMENT(bool, ShowInputGridNumbers)
		SLATE_ARGUMENT(bool, ShowOutputGridNumbers)

		SLATE_ARGUMENT(FLinearColor, NormalAreaColor)
		SLATE_ARGUMENT(FLinearColor, NormalBoundsColor)
		SLATE_ATTRIBUTE(bool, GridSnap)
		SLATE_ATTRIBUTE(int32, GridSize)
		SLATE_ARGUMENT(FLinearColor, GridColor)
		SLATE_ARGUMENT(FLinearColor, ExtendedGridColor)
		SLATE_ARGUMENT(float, CurveThickness)
		SLATE_ARGUMENT(FLinearColor, CurveColor)
		SLATE_ATTRIBUTE(EEaseCurveToolOperation, Operation)

		SLATE_ATTRIBUTE(bool, CanEditCurve)
		SLATE_ATTRIBUTE(TOptional<FText>, ErrorMessage)

		SLATE_ATTRIBUTE(FText, StartText)
		SLATE_ATTRIBUTE(FText, StartTooltipText)
		SLATE_ATTRIBUTE(FText, EndText)
		SLATE_ATTRIBUTE(FText, EndTooltipText)

		SLATE_EVENT(FOnEaseCurveChanged, OnTangentsChanged)
		SLATE_EVENT(FOnGetContent, GetContextMenuContent)
		SLATE_EVENT(FOnKeyDown, OnKeyDown)
		SLATE_EVENT(FSimpleDelegate, OnDragStart)
		SLATE_EVENT(FSimpleDelegate, OnDragEnd)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

	virtual ~SEaseCurveEditor() override;

	static FVector2d CalcTangentDir(const double InTangent);
	static float CalcTangent(const FVector2d& InHandleDelta);

	void ZoomToFit();

	/* Set flag that allows scrolling up/down over the widget from the outside without it handling the scroll wheel event. */
	void SetRequireFocusToZoom(const bool bInRequireFocusToZoom);

	FKeyHandle GetSelectedKeyHandle() const { return SelectedTangent.KeyHandle; }

	bool HasDrawErrorMessage() const;
	bool CanEditCurve() const;

protected:
	/** Represents the different states of a drag operation */
	enum class EDragState
	{
		/** There is no active drag operation. */
		None,
		/** The user has clicked a mouse button, but has not moved more then the drag threshold. */
		PreDrag,
		/** The user is dragging a selected tangent handle. */
		DragTangent,
		/** The user is performing a marquee selection of keys. */
		MarqueeSelect,
		/** The user is panning the curve view. */
		Pan,
		/** The user is zooming the curve view. */
		Zoom
	};

	struct FSelectedTangent
	{
		FSelectedTangent()
			: KeyHandle(FKeyHandle::Invalid()), bIsArrival(false)
		{}
		FSelectedTangent(const FKeyHandle InKey, const bool bInIsArrival)
			: KeyHandle(InKey), bIsArrival(bInIsArrival)
		{}

		bool operator == (const FSelectedTangent& InOther) const
		{
			return (KeyHandle == InOther.KeyHandle) && (bIsArrival == InOther.bIsArrival);
		}

		bool IsValid() const
		{
			return KeyHandle != FKeyHandle::Invalid();
		}

		FKeyHandle KeyHandle;

		/** Indicates if it is the arrival tangent, or the leave tangent */
		bool bIsArrival;
	};

	FVector2d GetArriveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo, const FKeyHandle& InKeyHandle) const;
	FVector2d GetArriveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo
		, const FVector2d& InKeyPosition, const float InTangent, const float InWeight) const;
	
	FVector2d GetLeaveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo, const FKeyHandle& InKeyHandle) const;
	FVector2d GetLeaveTangentScreenLocation(const FTrackScaleInfo& InScaleInfo
		, const FVector2d& InKeyPosition, const float InTangent, const float InWeight) const;

	void ClearSelection();

	int32 PaintGrid(const FTrackScaleInfo& InScaleInfo
		, const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const FSlateRect& InMyCullingRect
		, const ESlateDrawEffect InDrawEffects) const;

	void PaintNormalBounds(const FTrackScaleInfo& InScaleInfo
		, const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const FSlateRect& InMyCullingRect
		, const ESlateDrawEffect InDrawEffects
		, const FWidgetStyle& InWidgetStyle) const;

	void PaintCurve(const FTrackScaleInfo& InScaleInfo
		, const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FSlateRect& MyCullingRect
		, const ESlateDrawEffect DrawEffects
		, const FWidgetStyle& InWidgetStyle) const;

	int32 PaintKeys(const FTrackScaleInfo& InScaleInfo
		, const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const ESlateDrawEffect InDrawEffects
		, const FWidgetStyle& InWidgetStyle) const;

	int32 PaintTangentHandle(const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const ESlateDrawEffect InDrawEffects
		, const FWidgetStyle& InWidgetStyle
		, const FVector2D& InKeyLocation
		, const FVector2D& InTangentLocation
		, const bool bInSelected) const;

	int32 PaintSelectionText(const FTrackScaleInfo& InScaleInfo
		, const FGeometry& InGeometry
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const ESlateDrawEffect InDrawEffects) const;

	//~ Begin FEditorUndoClient
	virtual void PostUndo(const bool bInSuccess) override;
	virtual void PostRedo(const bool bInSuccess) override;
	//~ End FEditorUndoClient

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End FGCObject

	//~ Begin SWidget
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FVector2D ComputeDesiredSize(const float InLayoutScaleMultiplier) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void   OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const override;

	virtual int32 OnPaint(const FPaintArgs& InArgs
		, const FGeometry& InGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const FWidgetStyle& InWidgetStyle
		, const bool bInParentEnabled) const override;
	//~ End SWidget

	void TryStartDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void ProcessDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void EndDrag(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void ProcessClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void ZoomView(const FVector2D& InDelta);

	/* Generates the line(s) for rendering between KeyIndex and the following key. */
	void CreateLinesForSegment(const ERichCurveInterpMode InInterpMode
		, const ERichCurveTangentMode InTangentMode
		, const TPair<float, float>& InStartKeyTimeValue, const TPair<float, float>& InEndKeyTimeValue
		, TArray<FVector2D>& OutLinePoints, TArray<FLinearColor>& OutLineColors
		, const FTrackScaleInfo& InScaleInfo) const;

	void MoveSelectedTangent(const FTrackScaleInfo& InScaleInfo, const FVector2D& InScreenDelta);

	FVector2D SnapLocation(const FVector2D& InLocation) const;

	FText GetCurveToolTipInputText() const;
	FText GetCurveToolTipOutputText() const;

	TSharedRef<IToolTip> CreateCurveToolTip();
	void UpdateCurveToolTip(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	/** Set Default output values when range is too small */
	virtual void SetDefaultOutput(const float InMinZoomRange);

	void SetInputMinMax(const float InNewMin, const float InNewMax);
	void SetOutputMinMax(const float InNewMin, const float InNewMax);

	bool HitTestCurves(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) const;
	FSelectedTangent HitTestTangentHandles(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) const;
	bool HitTestKey(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent, const FVector2D& InInputPosition) const;

	TSharedPtr<FEaseCurveTool> Tool;
	TObjectPtr<UEaseCurve> EaseCurve;

	/** Minimum/Maximum input of data range  */
	TAttribute<TOptional<float>> DataMinInput;
	TAttribute<TOptional<float>> DataMaxInput;

	TAttribute<TOptional<FVector2D>> DesiredSize;
	TAttribute<FFrameRate> DisplayRate;

	bool bShowInputGridNumbers = false;
	bool bShowOutputGridNumbers = false;

	FLinearColor NormalAreaColor = FLinearColor::White;
	FLinearColor NormalBoundsColor = FLinearColor::White;
	TAttribute<bool> bGridSnap;
	TAttribute<int32> GridSize;
	FLinearColor GridColor = FLinearColor::White;
	FLinearColor ExtendedGridColor = FLinearColor::White;
	float CurveThickness = 1.f;
	FLinearColor CurveColor = FLinearColor::White;
	TAttribute<EEaseCurveToolOperation> Operation;

	TAttribute<bool> bCanEditCurve;
	TAttribute<TOptional<FText>> ErrorMessage;

	TAttribute<FText> StartText;
	TAttribute<FText> StartTooltipText;
	TAttribute<FText> EndText;
	TAttribute<FText> EndTooltipText;

	FOnEaseCurveChanged OnTangentsChanged;
	FOnGetContent GetContextMenuContent;
	FOnKeyDown OnKeyDownEvent;
	FSimpleDelegate OnDragStart;
	FSimpleDelegate OnDragEnd;

	float ViewMinInput = 0.f;
	float ViewMaxInput = 1.f;
	float ViewMinOutput = 0.f;
	float ViewMaxOutput = 1.f;

	FVector2D MouseDownLocation = FVector2D::ZeroVector;
	FVector2D MouseMoveLocation = FVector2D::ZeroVector;

	EDragState DragState = EDragState::None;
	/** The number of pixels which the mouse must move before a drag operation starts. */
	float DragThreshold = 4.f;
	/** A handle to the key which was clicked to start a key drag operation. */
	FKeyHandle DraggedKeyHandle;

	/** Tangent values at the beginning of a drag operation. */
	FEaseCurveTangents PreDragTangents;

	TSharedPtr<IToolTip> CurveToolTip;
	int8 ToolTipIndex = INDEX_NONE;

	FText CurveToolTipInputText;
	FText CurveToolTipOutputText;

	FEaseCurveTangents MovingTangents;
	FEaseCurveTangents LockTangents;

	bool bDraggingToolSizeValue = false;

	FSelectedTangent SelectedTangent;
	
	bool bRequireFocusToZoom = false;
};

} // namespace UE::EaseCurveTool
