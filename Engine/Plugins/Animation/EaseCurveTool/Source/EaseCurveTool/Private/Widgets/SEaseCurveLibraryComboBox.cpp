// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveLibraryComboBox.h"
#include "EaseCurveLibrary.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SEaseCurveLibraryComboBox"

namespace UE::EaseCurveTool
{

void SEaseCurveLibraryComboBox::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	WeakTool = InTool;

	PresetLibraryFactory = TStrongObjectPtr(NewObject<UEaseCurveLibraryFactory>());
	PresetLibraryFactory->AddToRoot();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton"))
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("OpenEaseCurveToolTabToolTip", "Open the Ease Curve Tool in its own tab window"))
			.Visibility(this, &SEaseCurveLibraryComboBox::GetOpenTabButtonVisibility)
			.OnClicked(this, &SEaseCurveLibraryComboBox::OpenEaseCurveToolTab)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(FEaseCurveStyle::Get().GetFloat(TEXT("ToolButton.ImageSize"))))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush(TEXT("Profiler.EventGraph.ExpandSelection")))
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UEaseCurveLibrary::StaticClass())
			.NewAssetFactories(TArray<UFactory*> { PresetLibraryFactory.Get() })
			.AllowClear(false)
			.AllowCreate(true)
			.DisplayThumbnail(true)
			.DisplayUseSelected(true)
			.DisplayBrowse(true)
			.ObjectPath(this, &SEaseCurveLibraryComboBox::GetSelectedPath)
			.OnObjectChanged(this, &SEaseCurveLibraryComboBox::HandleLibrarySelected)
		]
	];
}

FString SEaseCurveLibraryComboBox::GetSelectedPath() const
{
	const UEaseCurveLibrary* const PresetLibrary = WeakTool.IsValid() ? WeakTool.Pin()->GetPresetLibrary() : nullptr;
	if (PresetLibrary)
	{
		return PresetLibrary->GetPathName();
	}
	return FString();
}

void SEaseCurveLibraryComboBox::HandleLibrarySelected(const FAssetData& InAssetData)
{
	const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	// Allow null to clear the active preset library
	UEaseCurveLibrary* const PresetLibrary = Cast<UEaseCurveLibrary>(InAssetData.GetAsset());
	Tool->SetPresetLibrary(PresetLibrary);
}

EVisibility SEaseCurveLibraryComboBox::GetOpenTabButtonVisibility() const
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		return (!Tool->GetPresetLibrary() || Tool->IsToolTabVisible())
			? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

FReply SEaseCurveLibraryComboBox::OpenEaseCurveToolTab()
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		Tool->ShowHideToolTab(true);
	}
	return FReply::Handled();
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
