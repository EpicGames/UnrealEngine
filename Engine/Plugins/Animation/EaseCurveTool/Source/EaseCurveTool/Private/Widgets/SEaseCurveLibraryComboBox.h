// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveLibraryFactory.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::EaseCurveTool
{

class FEaseCurveTool;

/** A combo box widget for editing the preset library of an Ease Curve Tool instance */
class SEaseCurveLibraryComboBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurveLibraryComboBox)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

protected:
	FString GetSelectedPath() const;
	void HandleLibrarySelected(const FAssetData& InAssetData);

	EVisibility GetOpenTabButtonVisibility() const;
	FReply OpenEaseCurveToolTab();

	TWeakPtr<FEaseCurveTool> WeakTool;

	TStrongObjectPtr<UEaseCurveLibraryFactory> PresetLibraryFactory;
};

} // namespace UE::EaseCurveTool
