// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePreset.h"
#include "EaseCurveLibrary.h"
#include "EaseCurvePreset.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolUtils.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePreset"

namespace UE::EaseCurveTool
{

SEaseCurvePreset::~SEaseCurvePreset()
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		Tool->OnPresetLibraryChanged().RemoveAll(this);
	}

	if (UEaseCurveLibrary* const CurrentLibrary = WeakCurrentLibrary.Get())
	{
		CurrentLibrary->OnPresetChanged().RemoveAll(this);
	}
}

void SEaseCurvePreset::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	WeakTool = InTool;

	DisplayRate = InArgs._DisplayRate;
	OnPresetChanged = InArgs._OnPresetChanged;
	OnQuickPresetChanged = InArgs._OnQuickPresetChanged;
	OnGetNewPresetTangents = InArgs._OnGetNewPresetTangents;

	InTool->OnPresetLibraryChanged().AddSP(this, &SEaseCurvePreset::HandleLibraryChanged);

	ChildSlot
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]()
				{
					return bIsCreatingNewPreset ? 1 : 0;
				})
			+ SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SAssignNew(PresetComboBox, SEaseCurvePresetComboBox, InTool)
					.DisplayRate(DisplayRate)
					.OnPresetChanged(OnPresetChanged)
					.OnQuickPresetChanged(OnQuickPresetChanged)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton"))
					.VAlign(VAlign_Center)
					.ToolTipText(LOCTEXT("AddNewPresetToolTip", "Save the current ease curve as a new preset"))
					.IsEnabled_Lambda([this]()
						{
							return !PresetComboBox->HasSelection();
						})
					.OnClicked(this, &SEaseCurvePreset::OnCreateNewPresetClick)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(FEaseCurveStyle::Get().GetFloat(TEXT("ToolButton.ImageSize"))))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush(TEXT("Icons.Plus")))
					]
				]
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SAssignNew(NewPresetNameTextBox, SEditableTextBox)
					.OnKeyDownHandler(this, &SEaseCurvePreset::OnNewPresetKeyDownHandler)
					.OnTextCommitted(this, &SEaseCurvePreset::OnNewPresetTextCommitted)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton"))
					.VAlign(VAlign_Center)
					.ToolTipText(LOCTEXT("CancelNewPresetToolTip", "Cancels the current new ease curve preset operation"))
					.IsEnabled_Lambda([this]()
						{
							return !PresetComboBox->HasSelection();
						})
					.OnClicked(this, &SEaseCurvePreset::OnCancelNewPresetClick)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(FEaseCurveStyle::Get().GetFloat(TEXT("ToolButton.ImageSize"))))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush(TEXT("Icons.X")))
					]
				]
			]
		];
}

void SEaseCurvePreset::HandleLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InWeakLibrary)
{
	if (UEaseCurveLibrary* const CurrentLibrary = WeakCurrentLibrary.Get())
	{
		CurrentLibrary->OnPresetChanged().RemoveAll(this);
	}

	WeakCurrentLibrary = InWeakLibrary;

	if (InWeakLibrary.IsValid())
	{
		InWeakLibrary->OnPresetChanged().AddSP(this, &SEaseCurvePreset::HandlePresetChanged);
	}

	HandlePresetChanged();
}

void SEaseCurvePreset::HandlePresetChanged()
{
	if (!PresetComboBox.IsValid())
	{
		return;
	}

	FEaseCurvePreset Preset;
	const bool bHasSelectedItem = PresetComboBox->GetSelectedItem(Preset);

	PresetComboBox->Reload();

	if (bHasSelectedItem)
	{
		PresetComboBox->SetSelectedItem(Preset.GetHandle());
	}
}

FReply SEaseCurvePreset::OnCreateNewPresetClick()
{
	bIsCreatingNewPreset = true;

	FSlateApplication::Get().SetAllUserFocus(NewPresetNameTextBox);

	return FReply::Handled();
}

FReply SEaseCurvePreset::OnCancelNewPresetClick()
{
	bIsCreatingNewPreset = false;
	NewPresetNameTextBox->SetText(FText::GetEmpty());

	return FReply::Handled();
}

FReply SEaseCurvePreset::OnNewPresetKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		bIsCreatingNewPreset = false;
		NewPresetNameTextBox->SetText(FText::GetEmpty());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SEaseCurvePreset::OnNewPresetTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter && !InNewText.IsEmpty())
	{
		FEaseCurveTangents NewTangents;
		if (OnGetNewPresetTangents.IsBound() && OnGetNewPresetTangents.Execute(NewTangents))
		{
			if (UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool))
			{
				FEaseCurvePreset NewPreset;
				if (PresetLibrary->AddPresetToNewCategory(InNewText, NewTangents, NewPreset))
				{
					PresetComboBox->Reload();
					PresetComboBox->SetSelectedItem(NewPreset.GetHandle());
				}
			}
		}
	}

	bIsCreatingNewPreset = false;
	NewPresetNameTextBox->SetText(FText::GetEmpty());
}

void SEaseCurvePreset::ClearSelection()
{
	PresetComboBox->ClearSelection();
}

bool SEaseCurvePreset::SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle)
{
	return PresetComboBox->SetSelectedItem(InPresetHandle);
}

bool SEaseCurvePreset::SetSelectedItem(const FEaseCurveTangents& InTangents)
{
	return PresetComboBox->SetSelectedItem(InTangents);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
