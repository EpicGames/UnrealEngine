// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTangents.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SEaseCurvePresetComboBox.h"
#include "Widgets/SEaseCurvePresetGroup.h"

class FText;
struct FEaseCurvePreset;

namespace UE::EaseCurveTool
{

class SEaseCurvePreset : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetNewPresetTangents, FEaseCurveTangents& /*InTangents*/)

	SLATE_BEGIN_ARGS(SEaseCurvePreset)
		: _DisplayRate(FFrameRate(30, 1))
	{}
		/** Display rate used to draw the ease curve preview. */
		SLATE_ATTRIBUTE(FFrameRate, DisplayRate)
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)
		SLATE_EVENT(FOnPresetChanged, OnQuickPresetChanged)
		SLATE_EVENT(FOnGetNewPresetTangents, OnGetNewPresetTangents)
	SLATE_END_ARGS()

	virtual ~SEaseCurvePreset() override;

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

	void ClearSelection();
	
	bool SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle);
	bool SetSelectedItem(const FEaseCurveTangents& InTangents);

protected:
	void HandleLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InWeakLibrary);

	void HandlePresetChanged();

	FReply OnCreateNewPresetClick();
	FReply OnCancelNewPresetClick();
	
	FReply OnNewPresetKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);
	void OnNewPresetTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType);

	TWeakPtr<FEaseCurveTool> WeakTool;

	TAttribute<FFrameRate> DisplayRate;
	FOnPresetChanged OnPresetChanged;
	FOnPresetChanged OnQuickPresetChanged;
	FOnGetNewPresetTangents OnGetNewPresetTangents;

	TSharedPtr<SEaseCurvePresetComboBox> PresetComboBox;
	TSharedPtr<SEditableTextBox> NewPresetNameTextBox;

	bool bIsCreatingNewPreset = false;

	TWeakObjectPtr<UEaseCurveLibrary> WeakCurrentLibrary;
};

} // namespace UE::EaseCurveTool
