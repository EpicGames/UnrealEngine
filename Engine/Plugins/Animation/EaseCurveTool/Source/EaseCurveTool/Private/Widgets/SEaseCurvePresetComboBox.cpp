// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePresetComboBox.h"
#include "EaseCurvePreset.h"
#include "EaseCurveTool.h"
#include "Editor.h"
#include "DetailLayoutBuilder.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/SEaseCurvePresetList.h"
#include "Widgets/SEaseCurvePreview.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePresetComboBox"

namespace UE::EaseCurveTool
{
	
SEaseCurvePresetComboBox::~SEaseCurvePresetComboBox()
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		Tool->OnPresetLibraryChanged().RemoveAll(this);
	}
}

void SEaseCurvePresetComboBox::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	WeakTool = InTool;

	DisplayRate = InArgs._DisplayRate;
	bAllowEditMode = InArgs._AllowEditMode;
	OnPresetChanged = InArgs._OnPresetChanged;
	OnQuickPresetChanged = InArgs._OnQuickPresetChanged;

	InTool->OnPresetLibraryChanged().AddSP(this, &SEaseCurvePresetComboBox::HandlePresetLibraryChanged);

	ChildSlot
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SEaseCurvePresetComboBox::GeneratePresetDropdown)
			.OnMenuOpenChanged_Lambda([this](const bool bInOpening)
				{
					if (PresetList.IsValid())
					{
						PresetList->EnableEditMode(false);
					}
				})
			.ButtonContent()
			[
				SAssignNew(SelectedRowContainer, SBox)
			]
		];
}

void SEaseCurvePresetComboBox::HandlePresetLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InLibrary)
{
	ClearSelection();

	if (PresetList.IsValid())
	{
		PresetList->Reload();
	}
}

TSharedRef<SWidget> SEaseCurvePresetComboBox::GeneratePresetDropdown()
{
	check(WeakTool.IsValid());
	return SAssignNew(PresetList, SEaseCurvePresetList, WeakTool.Pin().ToSharedRef())
		.DisplayRate(DisplayRate)
		.OnPresetChanged(this, &SEaseCurvePresetComboBox::HandlePresetChanged)
		.OnQuickPresetChanged(OnQuickPresetChanged);
}

void SEaseCurvePresetComboBox::HandlePresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	SelectedItem = InPreset;

	GenerateSelectedRowWidget();

	OnPresetChanged.ExecuteIfBound(InPreset);
}

void SEaseCurvePresetComboBox::GenerateSelectedRowWidget()
{
	TSharedPtr<SWidget> OutRowWidget;

	if (!SelectedItem.IsValid())
	{
		OutRowWidget = SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(FText::FromString("Select Preset..."));
	}
	else
	{
		OutRowWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(0.f, 2.f, 5.f, 2.f)
			[
				SNew(SBorder)
				.BorderBackgroundColor(FStyleColors::White25)
				[
					SNew(SEaseCurvePreview)
					.PreviewSize(12.f)
					.CustomToolTip(true)
					.DisplayRate(DisplayRate.Get())
					.Tangents_Lambda([this]()
						{
							return SelectedItem.IsValid() ? SelectedItem->Tangents : FEaseCurveTangents();
						})
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 5.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FStyleColors::Foreground)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Text_Lambda([this]()
						{
							return SelectedItem.IsValid() ? SelectedItem->Name : FText::GetEmpty();
						})
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FStyleColors::White25)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Text_Lambda([this]()
						{
							return SelectedItem.IsValid() ? SelectedItem->Category : FText::GetEmpty();
						})
				]
			];
	}

	SelectedRowContainer->SetContent(OutRowWidget.ToSharedRef());
}

bool SEaseCurvePresetComboBox::HasSelection() const
{
	return SelectedItem.IsValid();
}

void SEaseCurvePresetComboBox::ClearSelection()
{
	SelectedItem.Reset();

	GenerateSelectedRowWidget();
}

bool SEaseCurvePresetComboBox::GetSelectedItem(FEaseCurvePreset& OutPreset) const
{
	if (!SelectedItem.IsValid())
	{
		return false;
	}

	OutPreset = *SelectedItem;

	return true;
}

bool SEaseCurvePresetComboBox::SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle)
{
	if (!PresetList.IsValid())
	{
		return false;
	}

	const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	const TSharedPtr<FEaseCurvePreset> FoundItem = PresetList->FindItem(InPresetHandle);
	if (!FoundItem.IsValid())
	{
		return false;
	}

	SelectedItem = FoundItem;

	GenerateSelectedRowWidget();

	return true;
}

bool SEaseCurvePresetComboBox::SetSelectedItem(const FEaseCurveTangents& InTangents)
{
	if (!PresetList.IsValid())
	{
		return false;
	}

	const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	const TSharedPtr<FEaseCurvePreset> FoundItem = PresetList->FindItemByTangents(InTangents);
	if (!FoundItem.IsValid())
	{
		return false;
	}

	SelectedItem = FoundItem;

	GenerateSelectedRowWidget();

	return true;
}

void SEaseCurvePresetComboBox::Reload()
{
	if (PresetList.IsValid())
	{
		PresetList->Reload();
	}
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
