// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTangents.h"
#include "SEaseCurvePresetList.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

class FText;
struct FEaseCurvePreset;

namespace UE::EaseCurveTool
{

class SEaseCurvePresetList;

class SEaseCurvePresetComboBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurvePresetComboBox)
		: _DisplayRate(FFrameRate(30, 1))
		, _AllowEditMode(true)
	{}
		SLATE_ARGUMENT(TWeakObjectPtr<UEaseCurveLibrary>, Library)
		/** Display rate used to draw the ease curve preview. */
		SLATE_ATTRIBUTE(FFrameRate, DisplayRate)
		SLATE_ARGUMENT(bool, AllowEditMode)
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)
		SLATE_EVENT(FOnPresetChanged, OnQuickPresetChanged)
	SLATE_END_ARGS()

	virtual ~SEaseCurvePresetComboBox() override;

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

	bool HasSelection() const;
	
	void ClearSelection();
	
	bool GetSelectedItem(FEaseCurvePreset& OutPreset) const;
	
	bool SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle);
	bool SetSelectedItem(const FEaseCurveTangents& InTangents);

	void Reload();

protected:
	void HandlePresetLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InLibrary);

	TSharedRef<SWidget> GeneratePresetDropdown();

	void HandlePresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset);

	void GenerateSelectedRowWidget();

	TWeakPtr<FEaseCurveTool> WeakTool;

	TAttribute<FFrameRate> DisplayRate;
	bool bAllowEditMode = false;
	FOnPresetChanged OnPresetChanged;
	FOnPresetChanged OnQuickPresetChanged;

	TSharedPtr<SEaseCurvePresetList> PresetList;
	TSharedPtr<SBox> SelectedRowContainer;

	TSharedPtr<FEaseCurvePreset> SelectedItem;
};

} // namespace UE::EaseCurveTool
