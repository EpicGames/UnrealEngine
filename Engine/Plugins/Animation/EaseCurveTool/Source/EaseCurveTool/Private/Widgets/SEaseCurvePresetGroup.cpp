// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePresetGroup.h"
#include "EaseCurvePreset.h"
#include "EaseCurvePresetDragDropOperation.h"
#include "EaseCurveStyle.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePresetGroup"

namespace UE::EaseCurveTool
{

void SEaseCurvePresetGroup::Construct(const FArguments& InArgs)
{
	CategoryName = InArgs._CategoryName;
	Presets = InArgs._Presets;
	SelectedPreset = InArgs._SelectedPreset;
	SearchText = InArgs._SearchText;
	bIsEditMode = InArgs._IsEditMode;
	DisplayRate = InArgs._DisplayRate;
	OnCategoryDelete = InArgs._OnCategoryDelete;
	OnCategoryRename = InArgs._OnCategoryRename;
	OnPresetDelete = InArgs._OnPresetDelete;
	OnPresetRename = InArgs._OnPresetRename;
	OnBeginPresetMove = InArgs._OnBeginPresetMove;
	OnEndPresetMove = InArgs._OnEndPresetMove;
	OnPresetClick = InArgs._OnPresetClick;
	OnSetQuickEase = InArgs._OnSetQuickEase;

	ChildSlot
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.Visibility(this, &SEaseCurvePresetGroup::GetEditModeVisibility)
				.BorderImage(this, &SEaseCurvePresetGroup::GetBorderImage)
			]
			+ SOverlay::Slot()
			.Padding(1.f)
			[
				SNew(SBox)
				.WidthOverride(140.f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.f)
					[
						ConstructHeader()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SListView<TSharedPtr<FEaseCurvePreset>>)
						.ListItemsSource(&VisiblePresets)
						.SelectionMode_Lambda([this]()
							{
								return IsEditMode() ? ESelectionMode::None : ESelectionMode::Single;
							})
						.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>(TEXT("PropertyTable.InViewport.ListView")))
						.OnGenerateRow(this, &SEaseCurvePresetGroup::GeneratePresetWidget)
					]
				]
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.Padding(1.f)
			[
				SNew(SButton)
				.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton.NoPad"))
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("EditModeDeleteTooltip", "Delete this category and the json file associated with it on disk"))
				.Visibility(this, &SEaseCurvePresetGroup::GetEditModeVisibility)
				.OnClicked(this, &SEaseCurvePresetGroup::HandleCategoryDelete)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(10.f))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(TEXT("Icons.Delete")))
				]
			]
		];
	
	SetSearchText(SearchText);
}

TSharedRef<SWidget> SEaseCurvePresetGroup::ConstructHeader()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(3.f, 2.f)
		[
			SNew(SBox)
			.MaxDesiredWidth(180.f)
			.Clipping(EWidgetClipping::ClipToBoundsAlways)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this]()
					{
						return IsEditMode() ? 1 : 0;
					})
				+ SWidgetSwitcher::Slot()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("Menu.Heading"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Text(CategoryName)
					.ToolTipText(this, &SEaseCurvePresetGroup::GetPresetNameTooltipText)
				]
				+ SWidgetSwitcher::Slot()
				[
					SAssignNew(RenameCategoryNameTextBox, SEditableTextBox)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.Text(CategoryName)
					.ToolTipText(this, &SEaseCurvePresetGroup::GetPresetNameTooltipText)
					.OnTextCommitted(this, &SEaseCurvePresetGroup::HandleCategoryRenameCommitted)
				]
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(3.f, 0.f)
		[
			SNew(SSeparator)
			.SeparatorImage(FAppStyle::Get().GetBrush(TEXT("Menu.Separator")))
			.Thickness(1.1f)
		];
}

TSharedRef<ITableRow> SEaseCurvePresetGroup::GeneratePresetWidget(const TSharedPtr<FEaseCurvePreset> InPreset, const TSharedRef<STableViewBase>& InOwnerTable)
{
	TSharedRef<SEaseCurvePresetGroupItem> NewPresetWidget = SNew(SEaseCurvePresetGroupItem, InOwnerTable)
		.Preset(InPreset)
		.IsEditMode(bIsEditMode)
		.DisplayRate(DisplayRate)
		.IsSelected(this, &SEaseCurvePresetGroup::IsSelected, InPreset)
		.OnDelete(this, &SEaseCurvePresetGroup::HandlePresetDelete)
		.OnRename(this, &SEaseCurvePresetGroup::HandlePresetRename)
		.OnBeginMove(this, &SEaseCurvePresetGroup::HandlePresetBeginMove)
		.OnEndMove(this, &SEaseCurvePresetGroup::HandlePresetEndMove)
		.OnClick(this, &SEaseCurvePresetGroup::HandlePresetClick)
		.OnSetQuickEase(this, &SEaseCurvePresetGroup::HandleSetQuickEase);
	PresetWidgetsMap.Add(InPreset, NewPresetWidget);

	return NewPresetWidget;
}

void SEaseCurvePresetGroup::SetSearchText(const FText& InText)
{
	SearchText = InText;

	VisiblePresets.Empty();

	const FString SearchString = SearchText.ToString();

	for (TSharedPtr<FEaseCurvePreset> Preset : Presets)
	{
		if (Preset.IsValid()
			&& (Preset->Name.ToString().Contains(SearchString)
				|| Preset->Category.ToString().Contains(SearchString)))
		{
			VisiblePresets.Add(Preset);
		}
	}

	VisiblePresets.Sort([](const TSharedPtr<FEaseCurvePreset>& InPresetA
		, const TSharedPtr<FEaseCurvePreset>& InPresetB)
		{
			return *InPresetA < *InPresetB;
		});

	const bool bIsVisible = SearchText.IsEmpty() || VisiblePresets.Num() > 0;
	SetVisibility(bIsVisible ? EVisibility::Visible : EVisibility::Collapsed);
}

int32 SEaseCurvePresetGroup::GetVisiblePresetCount() const
{
	return VisiblePresets.Num();
}

bool SEaseCurvePresetGroup::IsEditMode() const
{
	return bIsEditMode.Get(false);
}

EVisibility SEaseCurvePresetGroup::GetEditModeVisibility() const
{
	return IsEditMode() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SEaseCurvePresetGroup::GetBorderImage() const
{
	FName BrushName;

	if (bCanBeDroppedOn)
	{
		BrushName = bIsOverDifferentCategory ? TEXT("EditMode.Background.Over") : TEXT("EditMode.Background.Highlight");
	}
	else
	{
		BrushName = TEXT("EditMode.Background");
	}

	return FEaseCurveStyle::Get().GetBrush(BrushName);
}

FText SEaseCurvePresetGroup::GetPresetNameTooltipText() const
{
	return FText::Format(LOCTEXT("CategoryTooltipText", "Category: {0}"), CategoryName);
}

void SEaseCurvePresetGroup::HandleCategoryRenameCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	const FText NewCategoryName = InNewText;

	if (!NewCategoryName.IsEmpty() && !NewCategoryName.EqualToCaseIgnored(CategoryName)
		&& OnCategoryRename.IsBound() && OnCategoryRename.Execute(CategoryName, NewCategoryName))
	{
		CategoryName = NewCategoryName;
	}
	else
	{
		RenameCategoryNameTextBox->SetText(CategoryName);
	}
}

FReply SEaseCurvePresetGroup::HandleCategoryDelete() const
{
	if (OnCategoryDelete.IsBound())
	{
		OnCategoryDelete.Execute(CategoryName);
	}

	return FReply::Handled();
}

bool SEaseCurvePresetGroup::HandlePresetDelete(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	if (OnPresetDelete.IsBound() && OnPresetDelete.Execute(InPreset))
	{
		Presets.Remove(InPreset);
		VisiblePresets.Remove(InPreset);

		return true;
	}

	return false;
}

bool SEaseCurvePresetGroup::HandlePresetRename(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewName)
{
	if (OnPresetRename.IsBound() && OnPresetRename.Execute(InPreset, InNewName))
	{
		InPreset->Name = InNewName;

		return true;
	}
	else
	{
		PresetWidgetsMap[InPreset]->SetPreset(InPreset);

		return false;
	}
}

bool SEaseCurvePresetGroup::HandlePresetBeginMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategoryName) const
{
	if (OnBeginPresetMove.IsBound())
	{
		return OnBeginPresetMove.Execute(InPreset, InNewCategoryName);
	}

	return false;
}

bool SEaseCurvePresetGroup::HandlePresetEndMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategoryName) const
{
	if (OnEndPresetMove.IsBound() && OnEndPresetMove.Execute(InPreset, InNewCategoryName))
	{
		InPreset->Category = InNewCategoryName;

		return true;
	}

	return false;
}

bool SEaseCurvePresetGroup::HandlePresetClick(const TSharedPtr<FEaseCurvePreset>& InPreset) const
{
	if (OnPresetClick.IsBound())
	{
		OnPresetClick.Execute(InPreset);

		return true;
	}

	return false;
}

bool SEaseCurvePresetGroup::HandleSetQuickEase(const TSharedPtr<FEaseCurvePreset>& InPreset) const
{
	if (OnSetQuickEase.IsBound())
	{
		OnSetQuickEase.Execute(InPreset);

		return true;
	}

	return false;
}

void SEaseCurvePresetGroup::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	SCompoundWidget::OnDragEnter(InGeometry, InDragDropEvent);

	const TSharedPtr<FEaseCurvePresetDragDropOperation> Operation =
		InDragDropEvent.GetOperationAs<FEaseCurvePresetDragDropOperation>();

	if (Operation.IsValid())
	{
		bIsOverDifferentCategory = !CategoryName.EqualToCaseIgnored(Operation->GetPreset()->Category);
		bCanBeDroppedOn = bIsOverDifferentCategory;

		Operation->AddHoveredGroup(SharedThis(this));
	}
}

void SEaseCurvePresetGroup::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	SCompoundWidget::OnDragLeave(InDragDropEvent);

	const TSharedPtr<FEaseCurvePresetDragDropOperation> Operation =
		InDragDropEvent.GetOperationAs<FEaseCurvePresetDragDropOperation>();

	if (Operation.IsValid())
	{
		bCanBeDroppedOn = !CategoryName.EqualToCaseIgnored(Operation->GetPreset()->Category);
		bIsOverDifferentCategory = false;
	}
}

FReply SEaseCurvePresetGroup::OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	SCompoundWidget::OnDragOver(InGeometry, InDragDropEvent);

	const TSharedPtr<FEaseCurvePresetDragDropOperation> Operation =
		InDragDropEvent.GetOperationAs<FEaseCurvePresetDragDropOperation>();

	if (Operation.IsValid() && Operation->GetPreset().IsValid())
	{
		bIsOverDifferentCategory = !CategoryName.EqualToCaseIgnored(Operation->GetPreset()->Category);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SEaseCurvePresetGroup::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	SCompoundWidget::OnDrop(InGeometry, InDragDropEvent);

	ResetDragBorder();

	const TSharedPtr<FEaseCurvePresetDragDropOperation> Operation =
		InDragDropEvent.GetOperationAs<FEaseCurvePresetDragDropOperation>();

	if (Operation.IsValid())
	{
		if (OnEndPresetMove.IsBound() && OnEndPresetMove.Execute(Operation->GetPreset(), CategoryName))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SEaseCurvePresetGroup::ResetDragBorder()
{
	bCanBeDroppedOn = false;
	bIsOverDifferentCategory = false;
}

bool SEaseCurvePresetGroup::IsSelected(const TSharedPtr<FEaseCurvePreset> InPreset) const
{
	if (SelectedPreset.IsSet() && SelectedPreset.Get().IsValid())
	{
		return *InPreset == *SelectedPreset.Get();
	}

	return false;
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
