// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SEaseCurvePresetGroupItem.h"

class STableViewBase;
struct FEaseCurvePreset;

namespace UE::EaseCurveTool
{

DECLARE_DELEGATE_RetVal_OneParam(bool, FEaseCurveCategoryDeleteDelegate, const FText& /*InCategoryName*/)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FEaseCurveCategoryRenameDelegate, const FText& /*InOldName*/, const FText& /*InNewName*/)

class SEaseCurvePresetGroup : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurvePresetGroup)
		: _DisplayRate(FFrameRate(30, 1))
	{}
		SLATE_ARGUMENT(FText, CategoryName)
		SLATE_ARGUMENT(TArray<TSharedPtr<FEaseCurvePreset>>, Presets)
		SLATE_ATTRIBUTE(TSharedPtr<FEaseCurvePreset>, SelectedPreset)
		SLATE_ARGUMENT(FText, SearchText)
		SLATE_ATTRIBUTE(bool, IsEditMode)
		SLATE_ARGUMENT(FFrameRate, DisplayRate)
		SLATE_EVENT(FEaseCurveCategoryDeleteDelegate, OnCategoryDelete)
		SLATE_EVENT(FEaseCurveCategoryRenameDelegate, OnCategoryRename)
		SLATE_EVENT(FEaseCurvePresetDelegate, OnPresetDelete)
		SLATE_EVENT(FEaseCurvePresetRenameDelegate, OnPresetRename)
		SLATE_EVENT(FEaseCurvePresetMoveDelegate, OnBeginPresetMove)
		SLATE_EVENT(FEaseCurvePresetMoveDelegate, OnEndPresetMove)
		SLATE_EVENT(FEaseCurvePresetClickDelegate, OnPresetClick)
		SLATE_EVENT(FEaseCurvePresetDelegate, OnSetQuickEase)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetSearchText(const FText& InText);

	int32 GetVisiblePresetCount() const;

	bool IsEditMode() const;

	void ResetDragBorder();

	void NotifyCanDrop(const bool bInCanDrop) { bCanBeDroppedOn = bInCanDrop; }

	FText GetCategoryName() const { return CategoryName; }

	bool IsSelected(const TSharedPtr<FEaseCurvePreset> InPreset) const;

protected:
	TSharedRef<SWidget> ConstructHeader();

	TSharedRef<ITableRow> GeneratePresetWidget(const TSharedPtr<FEaseCurvePreset> InPreset, const TSharedRef<STableViewBase>& InOwnerTable);
	
	EVisibility GetEditModeVisibility() const;

	const FSlateBrush* GetBorderImage() const;

	FText GetPresetNameTooltipText() const;

	void HandleCategoryRenameCommitted(const FText& InNewText, ETextCommit::Type InCommitType);
	FReply HandleCategoryDelete() const;
	
	bool HandlePresetDelete(const TSharedPtr<FEaseCurvePreset>& InPreset);
	bool HandlePresetRename(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewName);
	bool HandlePresetBeginMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategoryName) const;
	bool HandlePresetEndMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategoryName) const;
	bool HandlePresetClick(const TSharedPtr<FEaseCurvePreset>& InPreset) const;
	bool HandleSetQuickEase(const TSharedPtr<FEaseCurvePreset>& InPreset) const;
	
	//~ Begin SWidget
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	//~ End SWidget

	FText CategoryName;
	TArray<TSharedPtr<FEaseCurvePreset>> Presets;
	FText SearchText;
	TAttribute<bool> bIsEditMode;
	FFrameRate DisplayRate;
	TAttribute<TSharedPtr<FEaseCurvePreset>> SelectedPreset;
	FEaseCurveCategoryDeleteDelegate OnCategoryDelete;
	FEaseCurveCategoryRenameDelegate OnCategoryRename;
	FEaseCurvePresetDelegate OnPresetDelete;
	FEaseCurvePresetRenameDelegate OnPresetRename;
	FEaseCurvePresetMoveDelegate OnBeginPresetMove;
	FEaseCurvePresetMoveDelegate OnEndPresetMove;
	FEaseCurvePresetClickDelegate OnPresetClick;
	FEaseCurvePresetDelegate OnSetQuickEase;

	TArray<TSharedPtr<FEaseCurvePreset>> VisiblePresets;

	TSharedPtr<SEditableTextBox> RenameCategoryNameTextBox;
	TMap<TSharedPtr<FEaseCurvePreset>, TSharedPtr<SEaseCurvePresetGroupItem>> PresetWidgetsMap;

	bool bCanBeDroppedOn = false;
	bool bIsOverDifferentCategory = false;
};

} // namespace UE::EaseCurveTool
