// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePresetGroupItem.h"
#include "EaseCurvePreset.h"
#include "EaseCurvePresetDragDropOperation.h"
#include "EaseCurveStyle.h"
#include "EaseCurveToolCommands.h"
#include "EaseCurveToolSettings.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SEaseCurvePreview.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePresetGroupItem"

namespace UE::EaseCurveTool
{

void SEaseCurvePresetGroupItem::Construct(const FArguments& InArgs, const TSharedPtr<STableViewBase>& InOwnerTableView)
{
	Preset = InArgs._Preset;
	bIsEditMode = InArgs._IsEditMode;
	IsSelected = InArgs._IsSelected;
	OnDelete = InArgs._OnDelete;
	OnRename = InArgs._OnRename;
	OnBeginMove = InArgs._OnBeginMove;
	OnEndMove = InArgs._OnEndMove;
	OnClick = InArgs._OnClick;
	OnSetQuickEase = InArgs._OnSetQuickEase;

	const FText ItemTooltipText = FText::Format(LOCTEXT("ItemTooltip", "{0}\n\nShift + Click to set as active quick preset")
		, Preset->Name);

	ChildSlot
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.Visibility(this, &SEaseCurvePresetGroupItem::GetBorderVisibility)
				.BorderImage(this, &SEaseCurvePresetGroupItem::GetBackgroundImage)
			]
			+ SOverlay::Slot()
			.Padding(1.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(3.f, 0.f)
				[
					SNew(SBox)
					.WidthOverride(160)
					.Clipping(EWidgetClipping::ClipToBoundsAlways)
					[
						SNew(SWidgetSwitcher)
						.WidgetIndex_Lambda([this]()
							{
								return IsEditMode() ? 1 : 0;
							})
						// Normal mode
						+ SWidgetSwitcher::Slot()
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), TEXT("Menu.Label"))
							.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Text(Preset->Name)
							.ToolTipText(ItemTooltipText)
						]
						// Edit mode
						+ SWidgetSwitcher::Slot()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
								.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton.NoPad"))
								.VAlign(VAlign_Center)
								.ToolTipText(LOCTEXT("EditModeDeleteTooltip", "Delete this category and the json file associated with it on disk"))
								.Visibility(this, &SEaseCurvePresetGroupItem::GetEditModeVisibility)
								.OnClicked(this, &SEaseCurvePresetGroupItem::HandleDeleteClick)
								[
									SNew(SImage)
									.DesiredSizeOverride(FVector2D(10.f))
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FAppStyle::GetBrush(TEXT("Icons.Delete")))
								]
							]
							+ SHorizontalBox::Slot()
							.Padding(2.f, 0.f, 0.f, 0.f)
							[
								SAssignNew(RenameTextBox, SEditableTextBox)
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
								.Text(Preset->Name)
								.ToolTipText(Preset->Name)
								.OnTextCommitted(this, &SEaseCurvePresetGroupItem::HandleRenameTextCommitted)
							]
						]
					]
				]
				// Quick Ease Button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SBorder)
					.Padding(0.f)
					.BorderImage(FEaseCurveStyle::Get().GetBrush(TEXT("ToolButton.Opaque")))
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton.NoPad"))
						.ToolTipText(this, &SEaseCurvePresetGroupItem::GetQuickPresetIconToolTip)
						.Visibility(this, &SEaseCurvePresetGroupItem::GetQuickPresetIconVisibility)
						.OnClicked(this, &SEaseCurvePresetGroupItem::HandleSetQuickEase)
						[
							SNew(SImage)
							.DesiredSizeOverride(FVector2D(10.f))
							.Image(FAppStyle::GetBrush(TEXT("Icons.Adjust")))
							.ColorAndOpacity(this, &SEaseCurvePresetGroupItem::GetQuickPresetIconColor)
						]
					]
				]
				// Preview Image
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SBorder)
					.BorderBackgroundColor(FStyleColors::White25)
					.Padding(2.f)
					.OnMouseButtonDown(this, &SEaseCurvePresetGroupItem::OnMouseButtonDown)
					[
						SNew(SEaseCurvePreview)
						.PreviewSize(20.f)
						.CurveThickness(1.5f)
						.Tangents(Preset->Tangents)
						.CustomToolTip(true)
						.BackgroundColor(FStyleColors::Dropdown.GetSpecifiedColor())
						.UnderCurveColor(FStyleColors::SelectInactive.GetSpecifiedColor())
						.DisplayRate(InArgs._DisplayRate)
					]
				]
			]
		];

	STableRow<TSharedPtr<FEaseCurvePreset>>::ConstructInternal(
		STableRow::FArguments()
		.Style(&FAppStyle::GetWidgetStyle<FTableRowStyle>(TEXT("ComboBox.Row")))
		.Padding(5.f)
		.ShowSelection(true)
		, InOwnerTableView.ToSharedRef());
}

void SEaseCurvePresetGroupItem::SetPreset(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	Preset = InPreset;
}

bool SEaseCurvePresetGroupItem::IsEditMode() const
{
	return bIsEditMode.Get(false);
}

EVisibility SEaseCurvePresetGroupItem::GetEditModeVisibility() const
{
	return IsEditMode() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SEaseCurvePresetGroupItem::GetBorderVisibility() const
{
	return ((bIsDragging && IsEditMode()) || (IsSelected.IsSet() && IsSelected.Get()))
		? EVisibility::Visible: EVisibility::Collapsed;
}

const FSlateBrush* SEaseCurvePresetGroupItem::GetBackgroundImage() const
{
	FName BrushName = NAME_None;

	if (bIsDragging)
	{
		BrushName = TEXT("EditMode.Background.Over");
	}
	else if (IsSelected.IsSet() && IsSelected.Get())
	{
		BrushName = TEXT("Preset.Selected");
	}

	return BrushName.IsNone() ? nullptr : FEaseCurveStyle::Get().GetBrush(BrushName);
}

void SEaseCurvePresetGroupItem::HandleRenameTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType) const
{
	const FText NewPresetName = InNewText;

	if (!NewPresetName.IsEmpty() && !NewPresetName.EqualToCaseIgnored(Preset->Name)
		&& OnRename.IsBound() && OnRename.Execute(Preset, NewPresetName))
	{
		Preset->Name = NewPresetName;
	}
	else
	{
		RenameTextBox->SetText(Preset->Name);
	}
}

FReply SEaseCurvePresetGroupItem::HandleDeleteClick() const
{
	if (OnDelete.IsBound())
	{
		OnDelete.Execute(Preset);
	}

	return FReply::Handled();
}

FReply SEaseCurvePresetGroupItem::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (IsEditMode())
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	if (OnClick.IsBound())
	{
		OnClick.Execute(Preset);
	}

	return FReply::Handled();
}

FReply SEaseCurvePresetGroupItem::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	TriggerEndMove();

	return STableRow::OnMouseButtonUp(InGeometry, InMouseEvent);
}

FReply SEaseCurvePresetGroupItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!IsEditMode())
	{
		return FReply::Unhandled();
	}

	TriggerBeginMove();

	const TSharedRef<FEaseCurvePresetDragDropOperation> Operation = MakeShared<FEaseCurvePresetDragDropOperation>(SharedThis(this), Preset);
	return FReply::Handled().BeginDragDrop(Operation);
}

FReply SEaseCurvePresetGroupItem::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	bIsDragging = false;

	return STableRow::OnDrop(InGeometry, InDragDropEvent);
}

void SEaseCurvePresetGroupItem::TriggerBeginMove()
{
	if (OnBeginMove.IsBound())
	{
		OnBeginMove.Execute(Preset, Preset->Category);
	}

	bIsDragging = true;
}

void SEaseCurvePresetGroupItem::TriggerEndMove()
{
	if (OnEndMove.IsBound())
	{
		OnEndMove.Execute(Preset, Preset->Category);
	}

	bIsDragging = false;
}

FSlateColor SEaseCurvePresetGroupItem::GetQuickPresetIconColor() const
{
	return (IsHovered() && IsQuickEasePreset()) ? FStyleColors::Select : FSlateColor::UseStyle();
}

EVisibility SEaseCurvePresetGroupItem::GetQuickPresetIconVisibility() const
{
	return (IsHovered() || IsQuickEasePreset()) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SEaseCurvePresetGroupItem::GetQuickPresetIconToolTip() const
{
	static const FText QuickEaseText = IsQuickEasePreset()
		? LOCTEXT("ActiveQuickEaseIconTooltip", "Active Quick Ease Preset")
		: LOCTEXT("QuickEaseIconTooltip", "Set to Active Quick Ease Preset");

	const FEaseCurveToolCommands& EaseCurveToolCommands = FEaseCurveToolCommands::Get();

	FText CommandText;

	if (EaseCurveToolCommands.QuickEase->GetFirstValidChord()->IsValidChord())
	{
		CommandText = FText::Format(LOCTEXT("QuickEaseIconInOutTooltip", "{0}{1} - Apply to Out (Leave) and In (Arrive) tangents\n")
			, CommandText, EaseCurveToolCommands.QuickEase->GetInputText());
	}

	if (EaseCurveToolCommands.QuickEaseIn->GetFirstValidChord()->IsValidChord())
	{
		CommandText = FText::Format(LOCTEXT("QuickEaseIconInTooltip", "{0}{1} - Apply to In (Arrive) tangent only\n")
			, CommandText, EaseCurveToolCommands.QuickEaseIn->GetInputText());
	}

	if (EaseCurveToolCommands.QuickEaseOut->GetFirstValidChord()->IsValidChord())
	{
		CommandText = FText::Format(LOCTEXT("QuickEaseIconOutTooltip", "{0}{1} - Apply to Out (Leave) tangent only\n")
			, CommandText, EaseCurveToolCommands.QuickEaseOut->GetInputText());
	}

	return CommandText.IsEmpty() ? QuickEaseText : FText::Format(LOCTEXT("QuickEasePresetIconTooltip", "{0}\n\n{1}")
		, QuickEaseText, CommandText);
}

FReply SEaseCurvePresetGroupItem::HandleSetQuickEase()
{
	if (OnSetQuickEase.IsBound())
	{
		OnSetQuickEase.Execute(Preset);
	}

	return FReply::Handled();
}

bool SEaseCurvePresetGroupItem::IsQuickEasePreset() const
{
	if (!Preset.IsValid())
	{
		return false;
	}
	
	const UEaseCurveToolSettings* const Settings = GetDefault<UEaseCurveToolSettings>();
	check(Settings);
	
	FEaseCurveTangents Tangents;
	
	if (!FEaseCurveTangents::FromString(Settings->GetQuickEaseTangents(), Tangents))
	{
		return false;
	}

	return Tangents.IsNearlyEqual(Preset->Tangents);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
