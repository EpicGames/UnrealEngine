// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FEaseCurvePreset;
class FText;
class SEditableTextBox;
class STableViewBase;

namespace UE::EaseCurveTool
{

DECLARE_DELEGATE_RetVal_OneParam(bool, FEaseCurvePresetDelegate, const TSharedPtr<FEaseCurvePreset>& /*InPreset*/)
DECLARE_DELEGATE_RetVal_OneParam(bool, FEaseCurvePresetClickDelegate, const TSharedPtr<FEaseCurvePreset>& /*InPreset*/)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FEaseCurvePresetRenameDelegate, const TSharedPtr<FEaseCurvePreset>& /*InPreset*/, const FText& /*InNewName*/)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FEaseCurvePresetMoveDelegate, const TSharedPtr<FEaseCurvePreset>& /*InPreset*/, const FText& /*InNewCategoryName*/)

class SEaseCurvePresetGroupItem : public STableRow<TSharedPtr<FEaseCurvePreset>>
{
public:
	SLATE_BEGIN_ARGS(SEaseCurvePresetGroupItem)
		: _DisplayRate(FFrameRate(30, 1))
	{}
		SLATE_ARGUMENT(TSharedPtr<FEaseCurvePreset>, Preset)
		SLATE_ATTRIBUTE(bool, IsEditMode)
		SLATE_ARGUMENT(FFrameRate, DisplayRate)
		SLATE_ATTRIBUTE(bool, IsSelected)
		SLATE_EVENT(FEaseCurvePresetClickDelegate, OnClick)
		SLATE_EVENT(FEaseCurvePresetDelegate, OnDelete)
		SLATE_EVENT(FEaseCurvePresetRenameDelegate, OnRename)
		SLATE_EVENT(FEaseCurvePresetDelegate, OnSetQuickEase)
		SLATE_EVENT(FEaseCurvePresetMoveDelegate, OnBeginMove)
		SLATE_EVENT(FEaseCurvePresetMoveDelegate, OnEndMove)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<STableViewBase>& InOwnerTableView);

	void SetPreset(const TSharedPtr<FEaseCurvePreset>& InPreset);

	bool IsEditMode() const;

	void TriggerBeginMove();
	void TriggerEndMove();

protected:
	void HandleRenameTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType) const;

	FReply HandleDeleteClick() const;

	EVisibility GetEditModeVisibility() const;

	EVisibility GetBorderVisibility() const;
	const FSlateBrush* GetBackgroundImage() const;

	FSlateColor GetQuickPresetIconColor() const;
	EVisibility GetQuickPresetIconVisibility() const;
	FText GetQuickPresetIconToolTip() const;

	bool IsQuickEasePreset() const;

	FReply HandleSetQuickEase();

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	//~ End SWidget

	TSharedPtr<FEaseCurvePreset> Preset;
	TAttribute<bool> bIsEditMode;
	TAttribute<bool> IsSelected;
	
	FEaseCurvePresetClickDelegate OnClick;
	FEaseCurvePresetDelegate OnDelete;
	FEaseCurvePresetRenameDelegate OnRename;
	FEaseCurvePresetDelegate OnSetQuickEase;
	
	FEaseCurvePresetMoveDelegate OnBeginMove;
	FEaseCurvePresetMoveDelegate OnEndMove;

	TSharedPtr<SEditableTextBox> RenameTextBox;

	bool bIsDragging = false;
};

} // namespace UE::EaseCurveTool
