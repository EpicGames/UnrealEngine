// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePresetList.h"
#include "EaseCurveLibrary.h"
#include "EaseCurvePreset.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolSettings.h"
#include "EaseCurveToolUtils.h"
#include "Internationalization/Text.h"
#include "SEaseCurvePresetGroup.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePresetList"

namespace UE::EaseCurveTool
{
	
SEaseCurvePresetList::~SEaseCurvePresetList()
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		Tool->OnPresetLibraryChanged().RemoveAll(this);
	}
}

void SEaseCurvePresetList::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	WeakTool = InTool;

	DisplayRate = InArgs._DisplayRate;
	bAllowEditMode = InArgs._AllowEditMode;
	OnPresetChanged = InArgs._OnPresetChanged;
	OnQuickPresetChanged = InArgs._OnQuickPresetChanged;

	InTool->OnPresetLibraryChanged().AddSP(this, &SEaseCurvePresetList::HandlePresetLibraryChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3.f)
		[
			SNew(SBox)
			.Visibility_Lambda([this]()
				{
					return (FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool) != nullptr)
						? EVisibility::Visible : EVisibility::Collapsed;
				})
			[
				GenerateSearchRowWidget()
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(3.f, 0.f, 3.f, 3.f)
		[
			SNew(SBox)
			.MaxDesiredHeight(960.f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SAssignNew(GroupWidgetsParent, SBox)
				]
			]
		]
	];

	Reload();
}

TSharedRef<SWidget> SEaseCurvePresetList::GenerateSearchRowWidget()
{
	static const FVector2D ButtonImageSize = FVector2D(FEaseCurveStyle::Get().GetFloat(TEXT("ToolButton.ImageSize")));

	TSharedRef<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SSearchBox)
			.HintText(LOCTEXT("SearchHintLabel", "Search"))
			.OnTextChanged(this, &SEaseCurvePresetList::HandleSearchTextChanged)
		];

	if (bAllowEditMode)
	{
		RowWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3.f, 0.f, 3.f, 0.f)
			[
				SNew(SCheckBox)
				.Style(FEaseCurveStyle::Get(), TEXT("ToolToggleButton"))
				.Padding(4.f)
				.ToolTipText(LOCTEXT("ToggleEditModeToolTip", "Enable editing of ease curve presets and categories"))
				.IsChecked_Lambda([this]()
					{
						return bEditMode.Get(false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged(this, &SEaseCurvePresetList::ToggleEditMode)
				[
					SNew(SImage)
					.DesiredSizeOverride(ButtonImageSize)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(TEXT("Icons.Edit")))
				]
			];

		RowWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FEaseCurveStyle::Get(), TEXT("ToolButton"))
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("CreateCategoryToolTip", "Create a new empty category"))
				.Visibility_Lambda([this]()
					{
						return bEditMode.Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.OnClicked(this, &SEaseCurvePresetList::CreateNewCategory)
				[
					SNew(SImage)
					.DesiredSizeOverride(ButtonImageSize)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(TEXT("Icons.Plus")))
				]
			];
	}

	return RowWidget;
}

void SEaseCurvePresetList::Reload()
{
	ReloadPresetItems();
	RegenerateGroupWrapBox();
}

void SEaseCurvePresetList::ReloadPresetItems()
{
	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return;
	}

	const TArray<FEaseCurvePreset>& PresetsToLoad = PresetLibrary->GetPresets();

	PresetItems.Reset(PresetsToLoad.Num());

	for (const FEaseCurvePreset& Preset : PresetsToLoad)
	{
		PresetItems.Add(MakeShared<FEaseCurvePreset>(Preset));
	}

	RegenerateGroupWrapBox();
}

void SEaseCurvePresetList::RegenerateGroupWrapBox()
{
	GroupWrapBox = SNew(SUniformWrapPanel)
		.HAlign(HAlign_Center)
		.SlotPadding(FMargin(2.f, 1.f))
		.EvenRowDistribution(true)
		.NumColumnsOverride_Lambda([this]()
			{
				if (UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool))
				{
					const int32 EaseCurveCategoryCount = PresetLibrary->GetCategories().Num();
					return FMath::Min(5, EaseCurveCategoryCount);
				}
				return 0;
			});

	if (UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool))
	{
		const TArray<FText>& PresetCategories = PresetLibrary->GetCategories();
		const int32 CurvePresetCount = PresetCategories.Num();

		GroupWidgets.Empty(CurvePresetCount);

		for (const FText& Category : PresetCategories)
		{
			TArray<TSharedPtr<FEaseCurvePreset>> GroupItems;
			TSet<FString> UniquePresetNames;

			for (const TSharedPtr<FEaseCurvePreset>& PresetItem : PresetItems)
			{
				if (PresetItem->Category.EqualToCaseIgnored(Category))
				{
					if (!UniquePresetNames.Contains(PresetItem->Name.ToString()))
					{
						GroupItems.Add(PresetItem);
					}
					UniquePresetNames.Add(PresetItem->Name.ToString());
				}
			}

			const TSharedRef<SEaseCurvePresetGroup> NewGroupWidget = SNew(SEaseCurvePresetGroup)
				.CategoryName(Category)
				.Presets(GroupItems)
				.SelectedPreset(SelectedItem)
				.IsEditMode(bEditMode)
				.DisplayRate(DisplayRate.Get())
				.OnCategoryDelete(this, &SEaseCurvePresetList::HandleCategoryDelete)
				.OnCategoryRename(this, &SEaseCurvePresetList::HandleCategoryRename)
				.OnPresetDelete(this, &SEaseCurvePresetList::HandlePresetDelete)
				.OnPresetRename(this, &SEaseCurvePresetList::HandlePresetRename)
				.OnBeginPresetMove(this, &SEaseCurvePresetList::HandleBeginPresetMove)
				.OnEndPresetMove(this, &SEaseCurvePresetList::HandleEndPresetMove)
				.OnPresetClick(this, &SEaseCurvePresetList::HandlePresetClick)
				.OnSetQuickEase(this, &SEaseCurvePresetList::HandleSetQuickEase);

			GroupWidgets.Add(NewGroupWidget);

			GroupWrapBox->AddSlot()
				.HAlign(HAlign_Left)
				[
					NewGroupWidget
				];
		}
	}

	UpdateGroupsContent();
}

void SEaseCurvePresetList::UpdateGroupsContent()
{
	auto GenerateNoPresetsWidget = [](const FText& InText)
	{
		return SNew(SBox)
			.WidthOverride(140.f)
			.HeightOverride(30.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.TextStyle(FAppStyle::Get(), TEXT("HintText"))
				.Text(InText)
			];
	};

	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		GroupWidgetsParent->SetContent(GenerateNoPresetsWidget(LOCTEXT("NoLibrarySelected", "No library selected")));
		return;
	}

	const TArray<FText> EaseCurveCategories = PresetLibrary->GetCategories();
	if (EaseCurveCategories.Num() == 0)
	{
		GroupWidgetsParent->SetContent(GenerateNoPresetsWidget(LOCTEXT("NoPresetsLabel", "No ease curve presets")));
		return;
	}

	if (!SearchText.IsEmpty())
	{
		int32 TotalVisiblePresets = 0;
		for (const TSharedPtr<SEaseCurvePresetGroup>& GroupWidget : GroupWidgets)
		{
			TotalVisiblePresets += GroupWidget->GetVisiblePresetCount();
		}
		if (TotalVisiblePresets == 0)
		{
			GroupWidgetsParent->SetContent(GenerateNoPresetsWidget(LOCTEXT("NoPresetsFoundLabel", "No ease curve presets found")));
			return;
		}
	}

	GroupWidgetsParent->SetContent(GroupWrapBox.ToSharedRef());
}

void SEaseCurvePresetList::HandlePresetLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InLibrary)
{
	ClearSelection();
	Reload();
}

void SEaseCurvePresetList::ToggleEditMode(const ECheckBoxState bInNewState)
{
	bEditMode.Set(bInNewState == ECheckBoxState::Checked ? true : false);

	RegenerateGroupWrapBox();
}

bool SEaseCurvePresetList::HasSelection() const
{
	return SelectedItem.IsValid();
}

bool SEaseCurvePresetList::GetSelectedItem(FEaseCurvePreset& OutPreset) const
{
	if (!SelectedItem.IsValid())
	{
		return false;
	}

	OutPreset = *SelectedItem;

	return true;
}

TSharedPtr<FEaseCurvePreset> SEaseCurvePresetList::FindItem(const FEaseCurvePresetHandle& InPresetHandle) const
{
	const TSharedPtr<FEaseCurvePreset>* FoundPresetItem = PresetItems.FindByPredicate([&InPresetHandle]
		(const TSharedPtr<FEaseCurvePreset>& InThisPreset)
		{
			return InThisPreset.IsValid() && *InThisPreset == InPresetHandle;
		});

	if (FoundPresetItem)
	{
		return *FoundPresetItem;
	}

	return nullptr;
}

TSharedPtr<FEaseCurvePreset> SEaseCurvePresetList::FindItemByTangents(const FEaseCurveTangents& InTangents, const double InErrorTolerance) const
{
	const TSharedPtr<FEaseCurvePreset>* FoundPresetItem = PresetItems.FindByPredicate([&InTangents, InErrorTolerance]
		(const TSharedPtr<FEaseCurvePreset>& InThisPreset)
		{
			return InThisPreset.IsValid() && InThisPreset->Tangents.IsNearlyEqual(InTangents, InErrorTolerance);
		});

	if (FoundPresetItem)
	{
		return *FoundPresetItem;
	}

	return nullptr;
}
	
void SEaseCurvePresetList::ClearSelection()
{
	SelectedItem.Reset();
	OnPresetChanged.ExecuteIfBound(nullptr);
}

bool SEaseCurvePresetList::SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle)
{
	const TSharedPtr<FEaseCurvePreset> FoundItem = FindItem(InPresetHandle);
	if (!FoundItem.IsValid())
	{
		return false;
	}

	SelectedItem = FoundItem;
	OnPresetChanged.ExecuteIfBound(SelectedItem);

	return true;
}

bool SEaseCurvePresetList::SetSelectedItem(const FEaseCurveTangents& InTangents)
{
	const TSharedPtr<FEaseCurvePreset> FoundItem = FindItemByTangents(InTangents);
	if (!FoundItem.IsValid())
	{
		return false;
	}

	SelectedItem = FoundItem;
	OnPresetChanged.ExecuteIfBound(SelectedItem);
	
	return true;
}

bool SEaseCurvePresetList::IsInEditMode() const
{
	return bEditMode.Get(false);
}

void SEaseCurvePresetList::EnableEditMode(const bool bInEnable)
{
	bEditMode.Set(bInEnable);
}

void SEaseCurvePresetList::HandleSearchTextChanged(const FText& InSearchText)
{
	SearchText = InSearchText;

	for (const TSharedPtr<SEaseCurvePresetGroup>& GroupWidget : GroupWidgets)
	{
		GroupWidget->SetSearchText(SearchText);
	}

	UpdateGroupsContent();
}

bool SEaseCurvePresetList::HandleCategoryDelete(const FText& InCategory)
{
	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return false;
	}

	if (!PresetLibrary->RemovePresetCategory(InCategory))
	{
		return false;
	}

	ReloadPresetItems();

	return true;
}

bool SEaseCurvePresetList::HandleCategoryRename(const FText& InCategory, const FText& InNewName)
{
	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return false;
	}

	if (!PresetLibrary->RenamePresetCategory(InCategory, InNewName))
	{
		return false;
	}

	ReloadPresetItems();

	return true;
}

bool SEaseCurvePresetList::HandlePresetDelete(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	if (!InPreset.IsValid())
	{
		return false;
	}

	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return false;
	}

	if (!PresetLibrary->RemovePreset(InPreset->GetHandle()))
	{
		return false;
	}

	ReloadPresetItems();

	return true;
}

bool SEaseCurvePresetList::HandlePresetRename(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewName)
{
	if (!InPreset.IsValid())
	{
		return false;
	}

	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return false;
	}

	if (!PresetLibrary->RenamePreset(InPreset->GetHandle(), InNewName))
	{
		return false;
	}

	ReloadPresetItems();

	return true;
}

bool SEaseCurvePresetList::HandleBeginPresetMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategory)
{
	for (const TSharedPtr<SEaseCurvePresetGroup>& GroupWidget : GroupWidgets)
	{
		if (!GroupWidget->GetCategoryName().EqualToCaseIgnored(InNewCategory))
		{
			GroupWidget->NotifyCanDrop(true);
		}
	}

	return true;
}

bool SEaseCurvePresetList::HandleEndPresetMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategory)
{
	if (!InPreset.IsValid())
	{
		return false;
	}

	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return false;
	}

	for (const TSharedPtr<SEaseCurvePresetGroup>& GroupWidget : GroupWidgets)
	{
		if (!GroupWidget->GetCategoryName().EqualToCaseIgnored(InNewCategory))
		{
			GroupWidget->NotifyCanDrop(false);
		}
	}

	if (!InPreset.IsValid() || InPreset->Category.EqualToCaseIgnored(InNewCategory))
	{
		return false;
	}

	if (!PresetLibrary->ChangePresetCategory(*InPreset, InNewCategory))
	{
		return false;
	}

	ReloadPresetItems();

	return true;
}

bool SEaseCurvePresetList::HandlePresetClick(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	if (InPreset.IsValid())
	{
		SetSelectedItem(InPreset->GetHandle());
		return true;
	}
	return false;
}

bool SEaseCurvePresetList::HandleSetQuickEase(const TSharedPtr<FEaseCurvePreset>& InPreset)
{
	UEaseCurveToolSettings* const EaseCurveToolSettings = GetMutableDefault<UEaseCurveToolSettings>();
	EaseCurveToolSettings->SetQuickEaseTangents(InPreset->Tangents.ToJson());
	EaseCurveToolSettings->SaveConfig();

	OnQuickPresetChanged.ExecuteIfBound(InPreset);

	return true;
}

FReply SEaseCurvePresetList::OnDeletePresetClick()
{
	if (!SelectedItem.IsValid())
	{
		return FReply::Handled();
	}

	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return FReply::Handled();
	}

	if (!PresetLibrary->RemovePreset(SelectedItem->GetHandle()))
	{
		return FReply::Handled();
	}

	ClearSelection();
	ReloadPresetItems();

	return FReply::Handled();
}

FReply SEaseCurvePresetList::CreateNewCategory()
{
	UEaseCurveLibrary* const PresetLibrary = FEaseCurveToolUtils::GetToolPresetLibrary(WeakTool);
	if (!PresetLibrary)
	{
		return FReply::Handled();
	}

	PresetLibrary->GenerateNewEmptyCategory();

	ReloadPresetItems();

	return FReply::Handled();
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
