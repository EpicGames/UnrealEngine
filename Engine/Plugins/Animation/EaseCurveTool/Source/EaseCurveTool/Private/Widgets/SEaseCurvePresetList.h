// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FEaseCurvePresetHandle;
class SBox;
class SUniformWrapPanel;
class UEaseCurveLibrary;
struct FEaseCurvePreset;
struct FEaseCurveTangents;

namespace UE::EaseCurveTool
{

class FEaseCurveTool;
class SEaseCurvePresetGroup;

DECLARE_DELEGATE_OneParam(FOnPresetChanged, const TSharedPtr<FEaseCurvePreset>&)

class SEaseCurvePresetList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurvePresetList)
		: _DisplayRate(FFrameRate(30, 1))
		, _AllowEditMode(true)
	{}
		/** Display rate used to draw the ease curve preview. */
		SLATE_ATTRIBUTE(FFrameRate, DisplayRate)
		SLATE_ARGUMENT(bool, AllowEditMode)
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)
		SLATE_EVENT(FOnPresetChanged, OnQuickPresetChanged)
	SLATE_END_ARGS()

	virtual ~SEaseCurvePresetList() override;

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

	bool HasSelection() const;

	bool GetSelectedItem(FEaseCurvePreset& OutPreset) const;

	void ClearSelection();

	bool SetSelectedItem(const FEaseCurvePresetHandle& InPresetHandle);
	bool SetSelectedItem(const FEaseCurveTangents& InTangents);

	TSharedPtr<FEaseCurvePreset> FindItem(const FEaseCurvePresetHandle& InPresetHandle) const;
	TSharedPtr<FEaseCurvePreset> FindItemByTangents(const FEaseCurveTangents& InTangents, const double InErrorTolerance = 0.01) const;

	bool IsInEditMode() const;
	void EnableEditMode(const bool bInEnable);

	void Reload();

protected:
	TSharedRef<SWidget> GenerateSearchRowWidget();

	void ReloadPresetItems();

	void RegenerateGroupWrapBox();
	void UpdateGroupsContent();

	void HandlePresetLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InLibrary);

	//FReply OnDeletePresetClick();

	FReply OnDeletePresetClick();
	FReply CreateNewCategory();

	void ToggleEditMode(const ECheckBoxState bInNewState);

	void HandleSearchTextChanged(const FText& InSearchText);

	bool HandleCategoryDelete(const FText& InCategory);
	bool HandleCategoryRename(const FText& InCategory, const FText& InNewName);
	bool HandlePresetDelete(const TSharedPtr<FEaseCurvePreset>& InPreset);
	bool HandlePresetRename(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewName);
	bool HandleBeginPresetMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategory);
	bool HandleEndPresetMove(const TSharedPtr<FEaseCurvePreset>& InPreset, const FText& InNewCategory);
	bool HandlePresetClick(const TSharedPtr<FEaseCurvePreset>& InPreset);
	bool HandleSetQuickEase(const TSharedPtr<FEaseCurvePreset>& InPreset);

	TWeakPtr<FEaseCurveTool> WeakTool;

	TAttribute<FFrameRate> DisplayRate;
	bool bAllowEditMode = false;
	FOnPresetChanged OnPresetChanged;
	FOnPresetChanged OnQuickPresetChanged;

	TSharedPtr<SBox> GroupWidgetsParent;
	TSharedPtr<SUniformWrapPanel> GroupWrapBox;
	TArray<TSharedPtr<SEaseCurvePresetGroup>> GroupWidgets;

	TArray<TSharedPtr<FEaseCurvePreset>> PresetItems;
	TSharedPtr<FEaseCurvePreset> SelectedItem;

	FText SearchText;

	TAttribute<bool> bEditMode;
};

} // namespace UE::EaseCurveTool
