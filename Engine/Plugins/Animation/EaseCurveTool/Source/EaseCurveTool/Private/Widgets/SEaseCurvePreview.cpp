// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePreview.h"
#include "Curves/CurveEvaluation.h"
#include "EaseCurveTool.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet/KismetMathLibrary.h"
#include "SCurveEditor.h"
#include "Styling/AppStyle.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/SEaseCurvePreviewToolTip.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePreview"

namespace UE::EaseCurveTool
{

void SEaseCurvePreview::Construct(const FArguments& InArgs)
{ 
	SetCanTick(InArgs._Animate);

	Tangents = InArgs._Tangents;
	PreviewSize = InArgs._PreviewSize;
	CanExpandPreview = InArgs._CanExpandPreview;
	CurveThickness = InArgs._CurveThickness;
	BackgroundColor = InArgs._BackgroundColor;
	CurveColor = InArgs._CurveColor;
	StraightColor = InArgs._StraightColor;
	UnderCurveColor = InArgs._UnderCurveColor;
	bAnimate = InArgs._Animate;
	IdleAnimationLength = InArgs._IdleAnimationLength;
	DisplayRate = InArgs._DisplayRate;
	AnimationColor = InArgs._AnimationColor;
	AnimationSize = InArgs._AnimationSize;
	bDrawMotionTrails = InArgs._DrawMotionTrails;
	MotionTrailColor = InArgs._MotionTrailColor;
	MotionTrailColorFade = InArgs._MotionTrailColorFade;
	MotionTrailSize = InArgs._MotionTrailSize;
	MotionTrailFadeLength = InArgs._MotionTrailFadeLength;
	MotionTrailOffset = InArgs._MotionTrailOffset;

	if (InArgs._CustomToolTip)
	{
		ToolTipWidget = MakeShared<SEaseCurvePreviewToolTip>(InArgs);
		SetToolTip(ToolTipWidget);
	}
	else
	{
		SetToolTipText(SEaseCurvePreviewToolTip::GetToolTipText(Tangents.Get(FEaseCurveTangents())));
	}

	SetTangents(InArgs._Tangents);
}

FEaseCurveTangents SEaseCurvePreview::GetTangents() const
{
	return Tangents.Get(FEaseCurveTangents());
}

void SEaseCurvePreview::SetTangents(const TAttribute<FEaseCurveTangents>& InTangents)
{
	Tangents = InTangents;

	// Save min and max to compute extra screen space if the curve goes beyond normal limits (0 - 1).
	float MinInputValue = 0.f;
	float MaxInputValue = 1.f;

	// Cache motion trails
	const FEaseCurveTangents DrawTangents = Tangents.Get(FEaseCurveTangents());
	const float SecondsPerFrame = DisplayRate.AsInterval();
	const int32 FrameCount = DisplayRate.AsFrameNumber(1.f).Value;

	MotionTrails.Empty(FrameCount + 1);

	for (float Time = 0; Time < 1.f; Time += SecondsPerFrame)
	{
		const float CurrentValue = WeightedEvalForTwoKeys(DrawTangents, Time);

		FMotionTrail NewMotionTrail;
		NewMotionTrail.Time = Time;
		NewMotionTrail.Color = FLinearColor(1.f, 1.f, 1.f, 1.f);
		NewMotionTrail.Location = FVector2D(CurrentValue, 0.f);
		MotionTrails.Add(MoveTemp(NewMotionTrail));

		MinInputValue = FMath::Min(CurrentValue, MinInputValue);
		MaxInputValue = FMath::Max(CurrentValue, MaxInputValue);
	}

	FMotionTrail LastMotionTrail;
	LastMotionTrail.Time = 1.f;
	LastMotionTrail.Color = FLinearColor(1.f, 1.f, 1.f, 1.f);
	LastMotionTrail.Location = FVector2D(1.f, 0.f);
	MotionTrails.Add(MoveTemp(LastMotionTrail));

	BelowZeroValue = FMath::Abs(MinInputValue);
	AboveOneValue = MaxInputValue - 1.f;
}

FVector2D SEaseCurvePreview::ComputeDesiredSize(const float InLayoutScaleMultiplier) const
{
	float AdditionalHeight = CanExpandPreview ? (BelowZeroValue + AboveOneValue) : 0.f;
	AdditionalHeight = FMath::Min(PreviewSize, AdditionalHeight);
	return FVector2D(PreviewSize) + FVector2D(AdditionalHeight * PreviewSize);
}

FReply SEaseCurvePreview::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FPlatformApplicationMisc::ClipboardCopy(*Tangents.Get(FEaseCurveTangents()).ToJson());

		FEaseCurveTool::ShowNotificationMessage(LOCTEXT("EaseCurveToolTangentsCopied", "Ease Curve Tool Tangents Copied!"));

		return FReply::Handled();
	}

	return SLeafWidget::OnMouseButtonDown(InGeometry, InMouseEvent);
}

void SEaseCurvePreview::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SLeafWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (IdleTimerHandle.IsValid()) // Fade out all animations
	{
		CurrentAnimateTime = 1.f;

		// Fade out animation and motion trails
		const double IdlePlayTime = FMath::Min(IdleAnimationLength, InCurrentTime - MotionTrailFadeStartTime);
		const double NewOpacity = UKismetMathLibrary::MapRangeClamped(IdlePlayTime, 0.f, IdleAnimationLength, 1.f, 0.f);

		AnimationColor.A = NewOpacity;

		if (bDrawMotionTrails)
		{
			const FLinearColor LerpColor = FLinearColor::LerpUsingHSV(FLinearColor(0.f, 0.f, 0.f, 0.f), AnimationColor, NewOpacity);

			for (FMotionTrail& MotionTrail : MotionTrails)
			{
				MotionTrail.Color = LerpColor;
			}
		}
	}
	else // Animate curve along path
	{
		if (CurrentAnimateTime >= 1.f)
		{
			CurrentAnimateTime = 1.f;
			MotionTrailFadeStartTime = InCurrentTime;

			IdleTimerHandle = RegisterActiveTimer(IdleAnimationLength, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime) -> EActiveTimerReturnType
				{
					IdleTimerHandle.Reset();

					CurrentAnimateTime = 0.f;

					if (bDrawMotionTrails)
					{
						for (FMotionTrail& MotionTrail : MotionTrails)
						{
							MotionTrail.Color = FLinearColor::White;
						}
					}

					return EActiveTimerReturnType::Stop;
				}));
		}
		else
		{
			CurrentAnimateTime += InDeltaTime;
			CurrentAnimateTime = FMath::Min(1.f, CurrentAnimateTime);

			const FTrackScaleInfo ScaleInfo(0.f, 1.f, 0.f, 1.f, FVector2f(PreviewSize)/*InAllottedGeometry.GetLocalSize()*/);
			const FEaseCurveTangents DrawTangents = Tangents.Get(FEaseCurveTangents());
			const float CurrentValue = WeightedEvalForTwoKeys(DrawTangents, CurrentAnimateTime);

			CurrentAnimateLocation = FVector2D(ScaleInfo.InputToLocalX(CurrentAnimateTime), ScaleInfo.OutputToLocalY(CurrentValue));

			AnimationColor.A = 1.f;

			if (bDrawMotionTrails)
			{
				for (FMotionTrail& MotionTrail : MotionTrails)
				{
					const float TimeSinceTrail = CurrentAnimateTime - MotionTrail.Time;
					const float ClampedTimeSinceTrail = FMath::Min(MotionTrailFadeLength, TimeSinceTrail);
					const float FadeOutAlpha = TimeSinceTrail <= MotionTrailFadeLength
						? UKismetMathLibrary::MapRangeClamped(ClampedTimeSinceTrail, 0.f, MotionTrailFadeLength, 0.f, 1.f)
						: 1.f;

					FLinearColor NewColor = FLinearColor::LerpUsingHSV(MotionTrailColor, MotionTrailColorFade, FadeOutAlpha);
					NewColor.A = MotionTrail.Color.A;

					MotionTrail.Color = NewColor;
				}
			}
		}
	}
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SEaseCurvePreview::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleImage(SharedThis(this)));
}
#endif

int32 SEaseCurvePreview::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bInParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	const FVector2D LocalSize = InAllottedGeometry.GetLocalSize();
	const FTrackScaleInfo ScaleInfo(0.f, 1.f, 0.f, 1.f, FVector2f(PreviewSize));

	const float ZeroOutputY = ScaleInfo.OutputToLocalY(0.f);
	const float OneOutputY = ScaleInfo.OutputToLocalY(1.f);

	int32 LayerId = InLayerId;

	const FVector2D AdditionalAboveHeight(BelowZeroValue * PreviewSize, AboveOneValue * PreviewSize);
	const FPaintGeometry AbovePaintGeometry = InAllottedGeometry.ToPaintGeometry(InAllottedGeometry.GetLocalSize(), FSlateLayoutTransform(AdditionalAboveHeight));

	// Draw background color
	if (BackgroundColor.IsSet())
	{
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, InAllottedGeometry.ToPaintGeometry()
			, FAppStyle::GetBrush(TEXT("WhiteTexture")), DrawEffects, BackgroundColor.GetValue());
	}

	// Draw 0 and 1 line
	{
		constexpr FLinearColor NormalLimitLineColor = FLinearColor(0.1f, 0.1f, 0.1f, 1.f);
		const TArray<FLinearColor> ValueLineColors = { FLinearColor::Transparent, NormalLimitLineColor, NormalLimitLineColor, FLinearColor::Transparent};
		const float LocalWidth = InAllottedGeometry.GetLocalSize().X;
		const float OneFifthWidth = LocalWidth * 0.2f;

		// Value = 0 line
		if (BelowZeroValue > 0.f)
		{
			TArray<FVector2D> ZeroValueLinePoints;
			ZeroValueLinePoints.Add(FVector2D(0.f, ZeroOutputY));
			ZeroValueLinePoints.Add(FVector2D(OneFifthWidth, ZeroOutputY));
			ZeroValueLinePoints.Add(FVector2D(LocalWidth - OneFifthWidth, ZeroOutputY));
			ZeroValueLinePoints.Add(FVector2D(LocalWidth, ZeroOutputY));
			TArray<FLinearColor> ZeroValueLineColors;
			FSlateDrawElement::MakeLines(OutDrawElements, LayerId++, AbovePaintGeometry
				, ZeroValueLinePoints, ValueLineColors, DrawEffects, FLinearColor::White, false, InAllottedGeometry.Scale);
		}

		// Value = 1 line
		if (AboveOneValue > 0.f)
		{
			TArray<FVector2D> OneValueLinePoints;
			OneValueLinePoints.Add(FVector2D(0.f, OneOutputY));
			OneValueLinePoints.Add(FVector2D(OneFifthWidth, OneOutputY));
			OneValueLinePoints.Add(FVector2D(LocalWidth - OneFifthWidth, OneOutputY));
			OneValueLinePoints.Add(FVector2D(LocalWidth, OneOutputY));
			FSlateDrawElement::MakeLines(OutDrawElements, LayerId++, AbovePaintGeometry
				, OneValueLinePoints, ValueLineColors, DrawEffects, FLinearColor::White, false, InAllottedGeometry.Scale);
		}
	}

	TArray<FVector2D> CurvePoints;
	TArray<FLinearColor> CurveColors;
	MakeCurveData(InAllottedGeometry, CurvePoints, CurveColors);

	// Draw under curve fill
	if (UnderCurveColor.IsSet())
	{
		for (const FVector2D& Point : CurvePoints)
		{
			const TArray<FVector2D> LinePoints = { FVector2D(Point.X, LocalSize.Y), Point };
			const TArray<FLinearColor> LineColors = { UnderCurveColor.GetValue(), UnderCurveColor.GetValue() };

			FSlateDrawElement::MakeLines(OutDrawElements, LayerId++, AbovePaintGeometry,
				LinePoints, LineColors, DrawEffects, FLinearColor::White, true, 2.0f * InAllottedGeometry.Scale);
		}
	}

	// Draw curve
	FSlateDrawElement::MakeLines(OutDrawElements, LayerId++, AbovePaintGeometry,
		CurvePoints, CurveColors, DrawEffects, FLinearColor::White, true, CurveThickness * InAllottedGeometry.Scale);

	// Draw animation
	if (bAnimate)
	{
		const FSlateBrush* ImageBrush = FAppStyle::GetBrush(TEXT("Icons.BulletPoint"));

		// Draw past motion trails
		if (bDrawMotionTrails)
		{
			for (const FMotionTrail& MotionTrail : MotionTrails)
			{
				if (MotionTrail.Time >= CurrentAnimateTime)
				{
					continue;
				}

				FVector2D MotionTrailLocation(ScaleInfo.InputToLocalX(MotionTrail.Location.X), ScaleInfo.OutputToLocalY(MotionTrail.Location.Y));
				MotionTrailLocation -= FVector2D(MotionTrailSize * 0.5f);
				MotionTrailLocation += MotionTrailOffset;
				MotionTrailLocation += AdditionalAboveHeight;

				FSlateDrawElement::MakeBox(OutDrawElements, LayerId++
					, InAllottedGeometry.ToPaintGeometry(FVector2D(MotionTrailSize), FSlateLayoutTransform(MotionTrailLocation))
					, ImageBrush, DrawEffects, MotionTrail.Color * InWidgetStyle.GetColorAndOpacityTint());
			}
		}

		// Draw current animation point
		const FVector2D TangentIconLocation = CurrentAnimateLocation - (AnimationSize * 0.5f);

		FSlateDrawElement::MakeBox(OutDrawElements, LayerId++
			, InAllottedGeometry.ToPaintGeometry(FVector2D(AnimationSize), FSlateLayoutTransform(TangentIconLocation + AdditionalAboveHeight))
			, ImageBrush, DrawEffects, AnimationColor * InWidgetStyle.GetColorAndOpacityTint());
	}

	return LayerId;
}

void SEaseCurvePreview::MakeCurveData(const FGeometry& InGeometry, TArray<FVector2D>& OutCurvePoints, TArray<FLinearColor>& OutCurveColors) const
{
	const FEaseCurveTangents DrawTangents = Tangents.Get(FEaseCurveTangents());
	const FTrackScaleInfo ScaleInfo(0.f, 1.f, 0.f, 1.f, FVector2f(PreviewSize)/*InAllottedGeometry.GetLocalSize()InGeometry.GetLocalSize()*/);

	// Clamp to screen to avoid massive slowdown when zoomed in
	float StartX = FMath::Max(ScaleInfo.InputToLocalX(0.f), 0.0f);
	const float EndX = FMath::Min(ScaleInfo.InputToLocalX(1.f), ScaleInfo.WidgetSize.X);
	
	constexpr float StepSize = 1.0f;
	float LastOutputY = 0.0f;
	
	for (; StartX < EndX; StartX += StepSize)
	{
		const float CurveIn = ScaleInfo.LocalXToInput(FMath::Min(StartX, EndX));
		const float CurveOut = WeightedEvalForTwoKeys(DrawTangents, CurveIn);

		const FVector2D ScreenPoint(ScaleInfo.InputToLocalX(CurveIn), ScaleInfo.OutputToLocalY(CurveOut));
		OutCurvePoints.Add(ScreenPoint);

		const float NewCurveStrength = CurveOut - LastOutputY;
		LastOutputY = CurveOut;

		const float Alpha = UKismetMathLibrary::MapRangeClamped(FMath::Abs(NewCurveStrength), 0.f, 0.03f, 0.f, 1.f);
		OutCurveColors.Add(FLinearColor::LerpUsingHSV(StraightColor, CurveColor, Alpha));
	}
}

float SEaseCurvePreview::WeightedEvalForTwoKeys(const FEaseCurveTangents& InTangents, const float InTime) const
{
	FRichCurveKey StartKey(0.f, 0.f);
	StartKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
	StartKey.TangentMode = ERichCurveTangentMode::RCTM_Break;
	StartKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedLeave;
	StartKey.ArriveTangent = 0.f;
	StartKey.ArriveTangentWeight = 0.f;
	StartKey.LeaveTangent = InTangents.Start;
	StartKey.LeaveTangentWeight = InTangents.StartWeight;

	FRichCurveKey EndKey(1.f, 1.f);
	EndKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
	EndKey.TangentMode = ERichCurveTangentMode::RCTM_Break;
	EndKey.TangentWeightMode = ERichCurveTangentWeightMode::RCTWM_WeightedArrive;
	EndKey.ArriveTangent = InTangents.End;
	EndKey.ArriveTangentWeight = InTangents.EndWeight;
	EndKey.LeaveTangent = 0.f;
	EndKey.LeaveTangentWeight = 0.f;
	
	return UE::Curves::EvalForTwoKeys(StartKey, EndKey, InTime);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
