// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SEaseCurvePreset.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FGeometry;

namespace UE::EaseCurveTool
{

class SEaseCurvePreviewToolTip;

class SEaseCurvePreview : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurvePreview)
		: _Tangents()
		, _PreviewSize(32.f)
		, _CanExpandPreview(false)
		, _CurveThickness(2.f)
		, _CurveColor(FLinearColor::White)
		, _StraightColor(FStyleColors::AccentBlue.GetSpecifiedColor())
		, _CustomToolTip(false)
		, _Animate(false)
		, _IdleAnimationLength(1.f)
		, _DisplayRate(FFrameRate(30, 1))
		, _AnimationColor(FLinearColor::White)
		, _AnimationSize(12.f)
		, _DrawMotionTrails(false)
		, _MotionTrailColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.f))
		, _MotionTrailColorFade(FLinearColor(0.15f, 0.15f, 0.15f, 1.f))
		, _MotionTrailSize(10.f)
		, _MotionTrailFadeLength(0.2f)
		, _MotionTrailOffset(FVector2D(0.f, 10.f))
	{}
		SLATE_ATTRIBUTE(FEaseCurveTangents, Tangents)
		SLATE_ARGUMENT(float, PreviewSize)
		/** If true, will expand the preview size to expand for curves that go outside of the normalized range. */
		SLATE_ARGUMENT(bool, CanExpandPreview)
		SLATE_ARGUMENT(float, CurveThickness)
	
		SLATE_ARGUMENT(TOptional<FLinearColor>, BackgroundColor)
		SLATE_ARGUMENT(FLinearColor, CurveColor)
		SLATE_ARGUMENT(FLinearColor, StraightColor)
		SLATE_ARGUMENT(TOptional<FLinearColor>, UnderCurveColor)

		/** If true, will show a custom tooltip showing an animated preview of the curve. */
		SLATE_ARGUMENT(bool, CustomToolTip)

		SLATE_ARGUMENT(bool, Animate)
		SLATE_ARGUMENT(bool, FlatAnimation)
		SLATE_ARGUMENT(float, IdleAnimationLength)
		SLATE_ARGUMENT(FFrameRate, DisplayRate)
		SLATE_ARGUMENT(FLinearColor, AnimationColor)
		SLATE_ARGUMENT(float, AnimationSize)
		SLATE_ARGUMENT(bool, DrawMotionTrails)
		SLATE_ARGUMENT(FLinearColor, MotionTrailColor)
		SLATE_ARGUMENT(FLinearColor, MotionTrailColorFade)
		SLATE_ARGUMENT(float, MotionTrailSize)
		SLATE_ARGUMENT(float, MotionTrailFadeLength)
		SLATE_ARGUMENT(FVector2D, MotionTrailOffset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FEaseCurveTangents GetTangents() const;
	void SetTangents(const TAttribute<FEaseCurveTangents>& InTangents);

	//~ Begin SWidget
#if WITH_ACCESSIBILITY
	virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	void MakeCurveData(const FGeometry& InGeometry, TArray<FVector2D>& OutCurvePoints, TArray<FLinearColor>& OutCurveColors) const;

	float WeightedEvalForTwoKeys(const FEaseCurveTangents& InTangents, const float InTime) const;

	//~ Begin SWidget
	virtual FVector2D ComputeDesiredSize(const float InLayoutScaleMultiplier) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	TAttribute<FEaseCurveTangents> Tangents;
	float PreviewSize = 0.f;
	bool CanExpandPreview = false;
	float CurveThickness = 1.f;

	TOptional<FLinearColor> BackgroundColor;
	FLinearColor CurveColor = FLinearColor::White;
	FLinearColor StraightColor = FLinearColor::White;
	TOptional<FLinearColor> UnderCurveColor;

	bool bAnimate = false;
	float IdleAnimationLength = 0.f;
	FFrameRate DisplayRate;
	FLinearColor AnimationColor = FLinearColor::White;
	float AnimationSize = 0.f;
	bool bDrawMotionTrails = false;
	FLinearColor MotionTrailColor = FLinearColor::White;
	FLinearColor MotionTrailColorFade = FLinearColor::White;
	float MotionTrailSize = 0.f;
	float MotionTrailFadeLength = 0.f;
	FVector2D MotionTrailOffset = FVector2D::ZeroVector;

	TSharedPtr<SEaseCurvePreviewToolTip> ToolTipWidget;

	/** The amount the curve is below value 0 in curve space. */
	float BelowZeroValue = 0.f;
	/** The amount the curve is above value 1 in curve space. */
	float AboveOneValue = 0.f;

	TSharedPtr<FActiveTimerHandle> IdleTimerHandle;

	double CurrentAnimateTime = 0.f;
	FVector2D CurrentAnimateLocation = FVector2D::ZeroVector;

	struct FMotionTrail
	{
		FVector2D Location = FVector2D::ZeroVector;
		float Time = 0.f;
		FLinearColor Color = FLinearColor(1.f, 1.f, 1.f, 1.f);
	};
	TArray<FMotionTrail> MotionTrails;

	double MotionTrailFadeStartTime = 0.f;
};

} // namespace UE::EaseCurveTool
