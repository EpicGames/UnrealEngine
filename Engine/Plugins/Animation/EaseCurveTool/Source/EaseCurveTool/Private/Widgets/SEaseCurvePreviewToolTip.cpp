// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurvePreviewToolTip.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/SEaseCurvePreset.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SEaseCurvePreviewToolTip"

namespace UE::EaseCurveTool
{

SEaseCurvePreviewToolTip::SEaseCurvePreviewToolTip(const SEaseCurvePreview::FArguments& InPreviewArgs, const TSharedPtr<SWidget>& InAdditionalContent)
	: PreviewArgs(InPreviewArgs), AdditionalContent(InAdditionalContent)
{
}

FText SEaseCurvePreviewToolTip::GetToolTipText(const FEaseCurveTangents& InTangents)
{
	return FText::Format(LOCTEXT("FullToolTipText", "{0}\n\nRight click to copy to Json values"), InTangents.ToDisplayText());
}

TSharedRef<SToolTip> SEaseCurvePreviewToolTip::CreateDefaultToolTip(const SEaseCurvePreview::FArguments& InPreviewArgs, const TSharedPtr<SWidget>& InAdditionalContent)
{
	const TSharedRef<SVerticalBox> ContentWidget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.f)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SColorBlock)
				.Color(FStyleColors::Background.GetSpecifiedColor())
			]
			+ SOverlay::Slot()
			.Padding(15.f, 15.f, 15.f, 20.f)
			[
				SNew(SEaseCurvePreview)
				.Tangents(InPreviewArgs._Tangents)
				.PreviewSize(256.f)
				.CanExpandPreview(true)
				.CurveThickness(InPreviewArgs._CurveThickness)
				.CurveColor(InPreviewArgs._CurveColor)
				.CustomToolTip(false)
				.Animate(true)
				.DisplayRate(InPreviewArgs._DisplayRate)
				.DrawMotionTrails(true)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 2.f, 15.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.ToolTipText(LOCTEXT("StartTangentTooltipText", "Start tangent and weight"))
				.Text(InPreviewArgs._Tangents.Get(FEaseCurveTangents()).GetStartTangentText())
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.ToolTipText(LOCTEXT("EndTangentTooltipText", "End tangent and weight"))
				.Text(InPreviewArgs._Tangents.Get(FEaseCurveTangents()).GetEndTangentText())
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 5.f, 15.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CubicBezierText", "Cubic Bezier Points:"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.ToolTipText(LOCTEXT("CubicBezierTooltipText", "Cubic bezier points"))
				.Text(InPreviewArgs._Tangents.Get(FEaseCurveTangents()).GetCubicBezierText())
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(20.f, 10.f, 0.f, 10.f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), TEXT("HintText"))
			.Text(LOCTEXT("ToolTipText", "Right click to copy to tangent values"))
		];

	if (InAdditionalContent.IsValid())
	{
		ContentWidget->AddSlot()
			.Padding(15.f, 10.f, 15.f, 0.f)
			[
				InAdditionalContent.ToSharedRef()
			];
	}

	return SNew(SToolTip)
		.TextMargin(1.0f)
		[
			ContentWidget
		];
}

void SEaseCurvePreviewToolTip::CreateToolTipWidget()
{
	if (ToolTipWidget.IsValid())
	{
		return;
	}

	ToolTipWidget = CreateDefaultToolTip(PreviewArgs, AdditionalContent);
}

TSharedRef<SWidget> SEaseCurvePreviewToolTip::AsWidget()
{
	CreateToolTipWidget();
	return ToolTipWidget.ToSharedRef();
}

TSharedRef<SWidget> SEaseCurvePreviewToolTip::GetContentWidget()
{
	CreateToolTipWidget();
	return ToolTipWidget->GetContentWidget();
}

void SEaseCurvePreviewToolTip::SetContentWidget(const TSharedRef<SWidget>& InContentWidget)
{
	CreateToolTipWidget();
	ToolTipWidget->SetContentWidget(InContentWidget);
}

void SEaseCurvePreviewToolTip::InvalidateWidget()
{
	ToolTipWidget.Reset();
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
