// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SEaseCurvePreview.h"

class SToolTip;
class SWidget;

namespace UE::EaseCurveTool
{

class SEaseCurvePreviewToolTip : public IToolTip
{
public:
	static FText GetToolTipText(const FEaseCurveTangents& InTangents);

	static TSharedRef<SToolTip> CreateDefaultToolTip(const SEaseCurvePreview::FArguments& InPreviewArgs, const TSharedPtr<SWidget>& InAdditionalContent = nullptr);

	SEaseCurvePreviewToolTip(const SEaseCurvePreview::FArguments& InPreviewArgs, const TSharedPtr<SWidget>& InAdditionalContent = nullptr);
	virtual ~SEaseCurvePreviewToolTip() override = default;

protected:
	void CreateToolTipWidget();

	void InvalidateWidget();

	//~ Begin IToolTip
	virtual TSharedRef<SWidget> AsWidget() override;
	virtual TSharedRef<SWidget> GetContentWidget() override;
	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget) override;
	virtual bool IsEmpty() const override { return false; }
	virtual bool IsInteractive() const override { return false; }
	virtual void OnOpening() override {}
	virtual void OnClosed() override {}
	//~ End IToolTip

	SEaseCurvePreview::FArguments PreviewArgs;
	TSharedPtr<SWidget> AdditionalContent;

	TSharedPtr<SToolTip> ToolTipWidget;
};

} // namespace UE::EaseCurveTool
