// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveTangents.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SEaseCurveTangents"

namespace UE::EaseCurveTool
{

void SEaseCurveTangents::Construct(const FArguments& InArgs)
{
	Tangents = InArgs._InitialTangents;
	OnStartTangentChanged = InArgs._OnStartTangentChanged;
	OnStartWeightChanged = InArgs._OnStartWeightChanged;
	OnEndTangentChanged = InArgs._OnEndTangentChanged;
	OnEndWeightChanged = InArgs._OnEndWeightChanged;
	OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
	OnEndSliderMovement = InArgs._OnEndSliderMovement;
	
	constexpr float WrapSize = 120.f;

	constexpr double MinTangent = -180.0f;
	constexpr double MaxTangent = 180.0f;
	constexpr double MinWeight = 0.0f;
	constexpr double MaxWeight = 10.0f;

	ChildSlot
	[
		SNew(SWrapBox)
		.UseAllottedSize(true)
		.HAlign(HAlign_Center)
		+ SWrapBox::Slot()
		.FillLineWhenSizeLessThan(WrapSize)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 1.f, 0.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(26.f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("OutLabel", "Out"))
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				ConstructTangentNumBox(LOCTEXT("OutTangentWeightLabel", "W")
					, LOCTEXT("OutTangentWeightToolTip", "Out Tangent Weight")
					, TAttribute<double>::CreateSP(this, &SEaseCurveTangents::GetStartTangentWeight)
					, SNumericEntryBox<double>::FOnValueChanged::CreateSP(this, &SEaseCurveTangents::OnStartTangentWeightSpinBoxChanged)
					, MinWeight, MaxWeight)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f, 0.f, 0.f)
			[
				ConstructTangentNumBox(LOCTEXT("OutTangentLabel", "T")
					, LOCTEXT("OutTangentToolTip", "Out Tangent")
					, TAttribute<double>::CreateSP(this, &SEaseCurveTangents::GetStartTangent)
					, SNumericEntryBox<double>::FOnValueChanged::CreateSP(this, &SEaseCurveTangents::OnStartTangentSpinBoxChanged)
					, MinTangent, MaxTangent)
			]
		]
		+ SWrapBox::Slot()
		.FillLineWhenSizeLessThan(WrapSize)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 1.f, 0.f, 0.f)
			[
				SNew(SBox)
				.WidthOverride(26.f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("InLabel", "In"))
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				ConstructTangentNumBox(LOCTEXT("InTangentWeightLabel", "W")
					, LOCTEXT("InTangentWeightToolTip", "In Tangent Weight")
					, TAttribute<double>::CreateSP(this, &SEaseCurveTangents::GetEndTangentWeight)
					, SNumericEntryBox<double>::FOnValueChanged::CreateSP(this, &SEaseCurveTangents::OnEndTangentWeightSpinBoxChanged)
					, MinWeight, MaxWeight)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f, 0.f, 0.f)
			[
				ConstructTangentNumBox(LOCTEXT("InTangentLabel", "T")
					, LOCTEXT("InTangentToolTip", "In Tangent")
					, TAttribute<double>::CreateSP(this, &SEaseCurveTangents::GetEndTangent)
					, SNumericEntryBox<double>::FOnValueChanged::CreateSP(this, &SEaseCurveTangents::OnEndTangentSpinBoxChanged)
					, MinTangent, MaxTangent)
			]
		]
	];
}

TSharedRef<SWidget> SEaseCurveTangents::ConstructTangentNumBox(const FText& InLabel
	, const FText& InToolTip
	, const TAttribute<double>& InValue
	, const SNumericEntryBox<double>::FOnValueChanged& InOnValueChanged
	, const TOptional<double>& InMinSliderValue
	, const TOptional<double>& InMaxSliderValue) const
{
	return SNew(SBox)
		.MaxDesiredWidth(100.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.MinDesiredWidth(8.f)
				.Margin(FMargin(2.f, 5.f, 2.f, 3.f))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(InLabel)
				.ToolTipText(InToolTip)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SSpinBox<double>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinSliderValue(InMinSliderValue)
				.MaxSliderValue(InMaxSliderValue)
				.Delta(0.00001f)
				.WheelStep(0.001f)
				.MinFractionalDigits(4)
				.MaxFractionalDigits(6)
				.MinDesiredWidth(70.f)
				.Value(InValue)
				.OnBeginSliderMovement(OnBeginSliderMovement)
				.OnEndSliderMovement(OnEndSliderMovement)
				.OnValueChanged_Lambda([this, InOnValueChanged](const double InNewValue)
					{
						InOnValueChanged.ExecuteIfBound(InNewValue);
					})
				.OnValueCommitted_Lambda([InOnValueChanged](const double InNewValue, ETextCommit::Type InCommitType)
					{
						InOnValueChanged.ExecuteIfBound(InNewValue);
					})
			]
		];
}

double SEaseCurveTangents::GetStartTangent() const
{
	return Tangents.Start;
}

double SEaseCurveTangents::GetStartTangentWeight() const
{
	return Tangents.StartWeight;
}

double SEaseCurveTangents::GetEndTangent() const
{
	return Tangents.End;
}

double SEaseCurveTangents::GetEndTangentWeight() const
{
	return Tangents.EndWeight;
}

void SEaseCurveTangents::SetTangents(const FEaseCurveTangents& InTangents)
{
	Tangents = InTangents;
}

void SEaseCurveTangents::OnStartTangentSpinBoxChanged(const double InNewValue)
{
	Tangents.Start = InNewValue;

	OnStartTangentChanged.Execute(InNewValue);
}

void SEaseCurveTangents::OnStartTangentWeightSpinBoxChanged(const double InNewValue)
{
	Tangents.StartWeight = InNewValue;
	
	OnStartWeightChanged.Execute(InNewValue);
}

void SEaseCurveTangents::OnEndTangentSpinBoxChanged(const double InNewValue)
{
	Tangents.End = InNewValue;

	OnEndTangentChanged.Execute(InNewValue);
}

void SEaseCurveTangents::OnEndTangentWeightSpinBoxChanged(const double InNewValue)
{
	Tangents.EndWeight = InNewValue;

	OnEndWeightChanged.Execute(InNewValue);
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
