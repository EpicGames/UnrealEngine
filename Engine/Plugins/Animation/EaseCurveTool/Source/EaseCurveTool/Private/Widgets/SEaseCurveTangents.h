// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTangents.h"
#include "Widgets/Input/SNumericEntryBox.h"

namespace UE::EaseCurveTool
{

class SEaseCurveTangents : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEaseCurveTangents)
	{}
		SLATE_ARGUMENT(FEaseCurveTangents, InitialTangents)
		SLATE_EVENT(SNumericEntryBox<double>::FOnValueChanged, OnStartTangentChanged)
		SLATE_EVENT(SNumericEntryBox<double>::FOnValueChanged, OnStartWeightChanged)
		SLATE_EVENT(SNumericEntryBox<double>::FOnValueChanged, OnEndTangentChanged)
		SLATE_EVENT(SNumericEntryBox<double>::FOnValueChanged, OnEndWeightChanged)
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)
		SLATE_EVENT(SNumericEntryBox<double>::FOnValueChanged, OnEndSliderMovement)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	double GetStartTangent() const;
	double GetStartTangentWeight() const;
	double GetEndTangent() const;
	double GetEndTangentWeight() const;

	void SetTangents(const FEaseCurveTangents& InTangents);

protected:
	TSharedRef<SWidget> ConstructTangentNumBox(const FText& InLabel
		, const FText& InToolTip
		, const TAttribute<double>& InValue
		, const SNumericEntryBox<double>::FOnValueChanged& InOnValueChanged
		, const TOptional<double>& InMinSliderValue
		, const TOptional<double>& InMaxSliderValue) const;

	void OnStartTangentSpinBoxChanged(const double InNewValue);
	void OnStartTangentWeightSpinBoxChanged(const double InNewValue);
	void OnEndTangentSpinBoxChanged(const double InNewValue);
	void OnEndTangentWeightSpinBoxChanged(const double InNewValue);

	FEaseCurveTangents Tangents;

	SNumericEntryBox<double>::FOnValueChanged OnStartTangentChanged;
	SNumericEntryBox<double>::FOnValueChanged OnStartWeightChanged;
	SNumericEntryBox<double>::FOnValueChanged OnEndTangentChanged;
	SNumericEntryBox<double>::FOnValueChanged OnEndWeightChanged;

	FSimpleDelegate OnBeginSliderMovement;
	SNumericEntryBox<double>::FOnValueChanged OnEndSliderMovement;
};

} // namespace UE::EaseCurveTool
