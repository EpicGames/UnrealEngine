// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEaseCurveTool.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "EaseCurvePreset.h"
#include "EaseCurveStyle.h"
#include "EaseCurveTool.h"
#include "EaseCurveToolSettings.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Math/UnrealMathUtility.h"
#include "Menus/EaseCurveToolContextMenu.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SEaseCurveEditor.h"
#include "Widgets/SEaseCurveLibraryComboBox.h"
#include "Widgets/SEaseCurvePreset.h"
#include "Widgets/SEaseCurveTangents.h"

#define LOCTEXT_NAMESPACE "SEaseCurveTool"

namespace UE::EaseCurveTool
{

void SEaseCurveTool::Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool)
{
	ToolMode = InArgs._ToolMode;
	ToolOperation = InArgs._ToolOperation;

	WeakTool = InTool;

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(0.f, 2.f, 0.f, 0.f)
			[
				SNew(SEaseCurveLibraryComboBox, InTool)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 2.f)
			.Expose(ToolWidgetSlot)
			[
				SNew(SVerticalBox)
				.Visibility_Lambda([this]()
					{
						if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
						{
							return Tool->GetPresetLibrary() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						return EVisibility::Collapsed;
					})
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(0.f, 1.f, 0.f, 0.f)
				[
					SAssignNew(CurvePresetWidget, SEaseCurvePreset, InTool)
					.OnPresetChanged(this, &SEaseCurveTool::OnPresetChanged)
					.OnQuickPresetChanged(this, &SEaseCurveTool::OnQuickPresetChanged)
					.OnGetNewPresetTangents(this, &SEaseCurveTool::OnGetNewPresetTangents)			
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(0.f, 3.f, 0.f, 0.f)
				.Expose(GraphEditorSlot)
				[
					ConstructCurveEditorPanel()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(0.f, 3.f, 0.f, 0.f)
				[
					SAssignNew(CurveTangentsWidget, SEaseCurveTangents)
					.InitialTangents(GetTangents())
					.OnStartTangentChanged(this, &SEaseCurveTool::OnStartTangentSpinBoxChanged)
					.OnStartWeightChanged(this, &SEaseCurveTool::OnStartTangentWeightSpinBoxChanged)
					.OnEndTangentChanged(this, &SEaseCurveTool::OnEndTangentSpinBoxChanged)
					.OnEndWeightChanged(this, &SEaseCurveTool::OnEndTangentWeightSpinBoxChanged)
					.OnBeginSliderMovement(this, &SEaseCurveTool::OnBeginSliderMovement)
					.OnEndSliderMovement(this, &SEaseCurveTool::OnEndSliderMovement)
				]
			]
		];

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	CurvePresetWidget->SetSelectedItem(InArgs._InitialTangents);
}

FVector2D SEaseCurveTool::ComputeDesiredSize(const float InScaleMultiplier) const
{
	if (ToolWidgetSlot && GraphEditorSlot)
	{
		if (GetDefault<UEaseCurveToolSettings>()->IsToolVisible())
		{
			ToolWidgetSlot->SetFillHeight(1.f);
			GraphEditorSlot->SetFillHeight(1.f);
		}
		else
		{
			ToolWidgetSlot->SetAutoHeight();
			GraphEditorSlot->SetAutoHeight();
		}
	}

	return SCompoundWidget::ComputeDesiredSize(InScaleMultiplier);
}

TSharedRef<SWidget> SEaseCurveTool::ConstructCurveEditorPanel()
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	check(EaseCurveTool.IsValid());

	CurrentGraphSize = GetDefault<UEaseCurveToolSettings>()->GetGraphSize();
	
	ContextMenu = MakeShared<FEaseCurveToolContextMenu>(EaseCurveTool->GetCommandList()
		, FEaseCurveToolOnGraphSizeChanged::CreateLambda([this](const int32 InNewSize)
			{
				CurrentGraphSize = InNewSize;
			}));

	return SNew(SBorder)
		.BorderImage_Lambda([this]
			{
				const bool bIsFocusedOrHovered = CurveEaseEditorWidget.IsValid()
					&& (CurveEaseEditorWidget->HasKeyboardFocus() || CurveEaseEditorWidget->IsHovered());
				const FName BrushName = bIsFocusedOrHovered ? TEXT("Editor.Border.Hover") : TEXT("Editor.Border");
				return FEaseCurveStyle::Get().GetBrush(BrushName);
			})
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(CurveEaseEditorWidget, SEaseCurveEditor, EaseCurveTool.ToSharedRef())
				.DisplayRate(this, &SEaseCurveTool::GetDisplayRate)
				.Operation(this, &SEaseCurveTool::GetToolOperation)
				.DesiredSize(this, &SEaseCurveTool::GetEditorSize)
				.CanEditCurve(this, &SEaseCurveTool::CanEditCurve)
				.ErrorMessage(this, &SEaseCurveTool::GetErrorMessage)
				.OnTangentsChanged(this, &SEaseCurveTool::HandleEditorTangentsChanged)
				.GridSnap_UObject(GetDefault<UEaseCurveToolSettings>(), &UEaseCurveToolSettings::GetGridSnap)
				.GridSize_UObject(GetDefault<UEaseCurveToolSettings>(), &UEaseCurveToolSettings::GetGridSize)
				.GetContextMenuContent(ContextMenu.ToSharedRef(), &FEaseCurveToolContextMenu::GenerateWidget)
				.StartText(this, &SEaseCurveTool::GetStartText)
				.StartTooltipText(this, &SEaseCurveTool::GetStartTooltipText)
				.EndText(this, &SEaseCurveTool::GetEndText)
				.EndTooltipText(this, &SEaseCurveTool::GetEndTooltipText)
				.OnKeyDown(this, &SEaseCurveTool::OnKeyDown)
				.OnDragStart(this, &SEaseCurveTool::OnEditorDragStart)
				.OnDragEnd(this, &SEaseCurveTool::OnEditorDragEnd)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.5f))
				.Image(&(FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>(TEXT("ScrollBox")).TopShadowBrush))
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.5f))
				.Image(&(FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>(TEXT("ScrollBox")).BottomShadowBrush))
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.5f))
				.Image(&(FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>(TEXT("ScrollBox")).LeftShadowBrush))
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.5f))
				.Image(&(FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>(TEXT("ScrollBox")).RightShadowBrush))
			]
		];
}

FFrameRate SEaseCurveTool::GetDisplayRate() const
{
	return WeakTool.IsValid() ? WeakTool.Pin()->GetDisplayRate() : FFrameRate();
}

EEaseCurveToolOperation SEaseCurveTool::GetToolOperation() const
{
	return WeakTool.IsValid() ? WeakTool.Pin()->GetToolOperation() : EEaseCurveToolOperation::InOut;
}

TOptional<FVector2D> SEaseCurveTool::GetEditorSize() const
{
	return GetDefault<UEaseCurveToolSettings>()->IsToolVisible()
		? TOptional<FVector2D>() : FVector2D(CurrentGraphSize);
}

bool SEaseCurveTool::CanEditCurve() const
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		const EEaseCurveToolError SelectionError = Tool->GetSelectionError();
		return SelectionError == EEaseCurveToolError::None
			|| SelectionError == EEaseCurveToolError::NoWeightedBrokenCubicTangents;
	}
	return false;
}

TOptional<FText> SEaseCurveTool::GetErrorMessage() const
{
	if (const TSharedPtr<FEaseCurveTool> Tool = WeakTool.Pin())
	{
		return Tool->GetSelectionErrorText();
	}
	return TOptional<FText>();
}

void SEaseCurveTool::HandleLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InWeakLibrary)
{

}

void SEaseCurveTool::HandleEditorTangentsChanged(const FEaseCurveTangents& InTangents) const
{
	SetTangents(InTangents, ToolOperation.Get(), true, true, true);
}

void SEaseCurveTool::OnEditorDragStart() const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	EaseCurveTool->BeginTransaction(LOCTEXT("EditorDragStartLabel", "Ease Curve Graph Drag"));
}

void SEaseCurveTool::OnEditorDragEnd() const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	EaseCurveTool->EndTransaction();

	if (!EaseCurveTool->HasCachedKeysToEase())
	{
		ResetTangentsAndNotify();
	}
}

void SEaseCurveTool::SetTangents(const FEaseCurveTangents& InTangents, EEaseCurveToolOperation InOperation
	, const bool bInSetEaseCurve, const bool bInBroadcastUpdate, const bool bInSetSequencerTangents) const
{
	if (CurvePresetWidget.IsValid() && !CurvePresetWidget->SetSelectedItem(InTangents))
	{
		CurvePresetWidget->ClearSelection();
	}

	if (CurveTangentsWidget.IsValid())
	{
		CurveTangentsWidget->SetTangents(InTangents);
	}

	// To change the graph UI tangents, we need to change the ease curve object tangents and the graph will reflect.
	if (bInSetEaseCurve)
	{
        if (const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin())
        {
        	EaseCurveTool->SetEaseCurveTangents(InTangents, InOperation, bInBroadcastUpdate, bInSetSequencerTangents);
        }
	}

	if (GetDefault<UEaseCurveToolSettings>()->GetAutoZoomToFit())
	{
		ZoomToFit();
	}
}

FEaseCurveTangents SEaseCurveTool::GetTangents() const
{
	if (const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin())
	{
		return EaseCurveTool->GetEaseCurveTangents();
	}
	return FEaseCurveTangents();
}

void SEaseCurveTool::OnStartTangentSpinBoxChanged(const double InNewValue) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	FEaseCurveTangents NewTangents = EaseCurveTool->GetEaseCurveTangents();
	NewTangents.Start = InNewValue;
	SetTangents(NewTangents, ToolOperation.Get(), true, true, true);
}

void SEaseCurveTool::OnStartTangentWeightSpinBoxChanged(const double InNewValue) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	FEaseCurveTangents NewTangents = EaseCurveTool->GetEaseCurveTangents();
	NewTangents.StartWeight = InNewValue;
	SetTangents(NewTangents, ToolOperation.Get(), true, true, true);
}

void SEaseCurveTool::OnEndTangentSpinBoxChanged(const double InNewValue) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	FEaseCurveTangents NewTangents = EaseCurveTool->GetEaseCurveTangents();
	NewTangents.End = InNewValue;
	SetTangents(NewTangents, ToolOperation.Get(), true, true, true);
}

void SEaseCurveTool::OnEndTangentWeightSpinBoxChanged(const double InNewValue) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	FEaseCurveTangents NewTangents = EaseCurveTool->GetEaseCurveTangents();
	NewTangents.EndWeight = InNewValue;
	SetTangents(NewTangents, ToolOperation.Get(), true, true, true);
}

void SEaseCurveTool::OnBeginSliderMovement()
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
    if (!EaseCurveTool.IsValid())
    {
    	return;
    }

	EaseCurveTool->BeginTransaction(LOCTEXT("SliderDragStartLabel", "Ease Curve Slider Drag"));
}

void SEaseCurveTool::OnEndSliderMovement(const double InNewValue)
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	EaseCurveTool->EndTransaction();
}

void SEaseCurveTool::OnPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return;
	}

	if (!EaseCurveTool->HasCachedKeysToEase())
	{
		ResetTangentsAndNotify();
		return;
	}

	if (InPreset.IsValid())
	{
		SetTangents(InPreset->Tangents, ToolOperation.Get(), true, true, true);

		EaseCurveTool->RecordPresetAnalytics(InPreset, TEXT("WithGraphEditor"));
	}

	FSlateApplication::Get().SetAllUserFocus(CurveEaseEditorWidget);
}

void SEaseCurveTool::OnQuickPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset) const
{
	FSlateApplication::Get().SetAllUserFocus(CurveEaseEditorWidget);
}

bool SEaseCurveTool::OnGetNewPresetTangents(FEaseCurveTangents& OutTangents) const
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return false;
	}

	OutTangents = EaseCurveTool->GetEaseCurveTangents();
	return true;
}

void SEaseCurveTool::UndoAction()
{
	if (GEditor)
	{
		GEditor->UndoTransaction();
	}
}

void SEaseCurveTool::RedoAction()
{
	if (GEditor)
	{
		GEditor->RedoTransaction();
	}
}

void SEaseCurveTool::ZoomToFit() const
{
	if (CurveEaseEditorWidget.IsValid())
	{
		CurveEaseEditorWidget->ZoomToFit();
	}
}

FKeyHandle SEaseCurveTool::GetSelectedKeyHandle() const
{
	if (CurveEaseEditorWidget.IsValid())
	{
		return CurveEaseEditorWidget->GetSelectedKeyHandle();
	}
	return FKeyHandle::Invalid();
}

FReply SEaseCurveTool::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const TSharedPtr<FEaseCurveTool> EaseCurveTool = WeakTool.Pin();
	if (!EaseCurveTool.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FUICommandList> CommandList = EaseCurveTool->GetCommandList();
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FText SEaseCurveTool::GetStartText() const
{
	return (ToolMode.Get(EEaseCurveToolMode::DualKeyEdit) == EEaseCurveToolMode::DualKeyEdit)
		? LOCTEXT("StartText", "Leave")
		: LOCTEXT("ArriveText", "Arrive");
}

FText SEaseCurveTool::GetStartTooltipText() const
{
	return (ToolMode.Get(EEaseCurveToolMode::DualKeyEdit) == EEaseCurveToolMode::DualKeyEdit)
		? LOCTEXT("StartTooltipText", "Start: The selected key's leave tangent")
		: LOCTEXT("ArriveTooltipText", "Arrive");
}

FText SEaseCurveTool::GetEndText() const
{
	return (ToolMode.Get(EEaseCurveToolMode::DualKeyEdit) == EEaseCurveToolMode::DualKeyEdit)
		? LOCTEXT("EndText", "Arrive")
		: LOCTEXT("LeaveText", "Leave");
}

FText SEaseCurveTool::GetEndTooltipText() const
{
	return (ToolMode.Get(EEaseCurveToolMode::DualKeyEdit) == EEaseCurveToolMode::DualKeyEdit)
		? LOCTEXT("EndTooltipText", "End: The next key's arrive tangent")
		: LOCTEXT("LeaveTooltipText", "Leave");
}

void SEaseCurveTool::ResetTangentsAndNotify() const
{
	CurvePresetWidget->ClearSelection();

	SetTangents(FEaseCurveTangents(), EEaseCurveToolOperation::InOut, true, true, false);

	FEaseCurveTool::ShowNotificationMessage(LOCTEXT("EqualValueKeys", "No different key values!"));
}

} // namespace UE::EaseCurveTool

#undef LOCTEXT_NAMESPACE
