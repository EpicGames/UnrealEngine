// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EaseCurveTool.h"
#include "EditorUndoClient.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FText;
struct FKeyHandle;

namespace UE::EaseCurveTool
{

class FEaseCurveToolContextMenu;
class SEaseCurveEditor;
class SEaseCurvePreset;
class SEaseCurveTangents;

class SEaseCurveTool
	: public SCompoundWidget
	, public FEditorUndoClient
{
public:
	static constexpr int32 DefaultGraphSize = 200;
	
	SLATE_BEGIN_ARGS(SEaseCurveTool)
		: _ToolMode(EEaseCurveToolMode::DualKeyEdit)
		, _ToolOperation(EEaseCurveToolOperation::InOut)
	{}
		SLATE_ATTRIBUTE(EEaseCurveToolMode, ToolMode)
		SLATE_ATTRIBUTE(EEaseCurveToolOperation, ToolOperation)
		SLATE_ARGUMENT(FEaseCurveTangents, InitialTangents)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FEaseCurveTool>& InTool);

	void SetTangents(const FEaseCurveTangents& InTangents, EEaseCurveToolOperation InOperation,
		const bool bInSetEaseCurve, const bool bInBroadcastUpdate, const bool bInSetSequencerTangents) const;

	FEaseCurveTangents GetTangents() const;

	FKeyHandle GetSelectedKeyHandle() const;

	void ZoomToFit() const;

	void ResetTangentsAndNotify() const;

protected:
	//~ Begin SWidget
	virtual FVector2D ComputeDesiredSize(const float InScaleMultiplier) const override;
	//~ End SWidget

	TSharedRef<SWidget> ConstructCurveEditorPanel();

	FFrameRate GetDisplayRate() const;
	EEaseCurveToolOperation GetToolOperation() const;
	TOptional<FVector2D> GetEditorSize() const;

	bool CanEditCurve() const;
	TOptional<FText> GetErrorMessage() const;

	void HandleLibraryChanged(const TWeakObjectPtr<UEaseCurveLibrary> InWeakLibrary);

	void HandleEditorTangentsChanged(const FEaseCurveTangents& InTangents) const;

	void OnStartTangentSpinBoxChanged(const double InNewValue) const;
	void OnStartTangentWeightSpinBoxChanged(const double InNewValue) const;
	void OnEndTangentSpinBoxChanged(const double InNewValue) const;
	void OnEndTangentWeightSpinBoxChanged(const double InNewValue) const;
	void OnBeginSliderMovement();
	void OnEndSliderMovement(const double InNewValue);

	void OnPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset) const;
	void OnQuickPresetChanged(const TSharedPtr<FEaseCurvePreset>& InPreset) const;
	bool OnGetNewPresetTangents(FEaseCurveTangents& OutTangents) const;

	void UndoAction();
	void RedoAction();

	void OnEditorDragStart() const;
	void OnEditorDragEnd() const;

	FText GetStartText() const;
	FText GetStartTooltipText() const;
	FText GetEndText() const;
	FText GetEndTooltipText() const;

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	TWeakPtr<FEaseCurveTool> WeakTool;

	TAttribute<EEaseCurveToolMode> ToolMode;
	TAttribute<EEaseCurveToolOperation> ToolOperation;

	TSharedPtr<SEaseCurvePreset> CurvePresetWidget;
	TSharedPtr<SEaseCurveEditor> CurveEaseEditorWidget;
	TSharedPtr<SEaseCurveTangents> CurveTangentsWidget;

	int32 CurrentGraphSize = DefaultGraphSize;

	TSharedPtr<FEaseCurveToolContextMenu> ContextMenu;

	SVerticalBox::FSlot* ToolWidgetSlot = nullptr;
	SVerticalBox::FSlot* GraphEditorSlot = nullptr;
};

} // namespace UE::EaseCurveTool
