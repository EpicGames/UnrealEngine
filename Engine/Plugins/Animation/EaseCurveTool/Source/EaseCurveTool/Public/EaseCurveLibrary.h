// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "EaseCurvePreset.h"
#include "Delegates/Delegate.h"
#include "EaseCurveLibrary.generated.h"

/**
 * Data asset that holds category and tangent data for ease curve presets
 */
UCLASS(MinimalAPI)
class UEaseCurveLibrary : public UDataAsset
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPresetChangedDelegate, const FEaseCurvePreset&);

public:
	/**
	 * Mainly used to compare clicked presets names for engine analytics, thus returns static category
	 * and preset names of the default presets instead of looking on disk for the actual default names.
	 */
	static TArray<FEaseCurvePreset> GetDefaultPresets();

	FSimpleMulticastDelegate& OnPresetChanged() { return PresetChangedDelegate; }

	bool DoesPresetExist(const FEaseCurvePresetHandle& InPresetHandle) const;

	const TArray<FEaseCurvePreset>& GetPresets() const;

	TArray<FText> GetCategories() const;

	TArray<FEaseCurvePreset> GetCategoryPresets(const FText& InCategory) const;

	bool AddPreset(const FEaseCurvePreset& InPreset);
	bool AddPresetToNewCategory(const FText& InName, const FEaseCurveTangents& InTangents, FEaseCurvePreset& OutNewPreset);

	bool RemovePreset(const FEaseCurvePresetHandle& InPresetHandle);

	bool ChangePresetCategory(const FEaseCurvePreset& InPreset, const FText& InNewCategory);

	bool FindPreset(const FEaseCurvePresetHandle& InPresetHandle, FEaseCurvePreset& OutPreset);
	bool FindPresetByTangents(const FEaseCurveTangents& InTangents
		, FEaseCurvePreset& OutPreset, const double InErrorTolerance = 0.01);

	bool DoesPresetCategoryExist(const FText& InCategory) const;
	bool RenamePresetCategory(const FText& InCategory, const FText& InNewCategory);
	bool RemovePresetCategory(const FText& InCategory);

	bool RenamePreset(const FEaseCurvePresetHandle& InPresetHandle, const FText& InNewPresetName);

	void SetToDefaultPresets();

	FText GenerateNewEmptyCategory();

	void Sort();

protected:
	UPROPERTY(EditAnywhere, Category = "Ease Curve Library")
	TArray<FEaseCurvePreset> Presets;

	UPROPERTY()
	TArray<FText> EmptyCategories;

	FSimpleMulticastDelegate PresetChangedDelegate;
};
