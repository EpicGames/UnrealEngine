// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "UObject/Object.h"
#include "EaseCurveSerializer.generated.h"

class FText;
class UEaseCurveLibrary;

#define UE_API EASECURVETOOL_API

/**
 * Derive from this class to make additional ease curve serializers capable of custom
 * saving and loading of presets.
 */
UCLASS(MinimalAPI, Abstract)
class UEaseCurveSerializer : public UObject
{
	GENERATED_BODY()

public:
	static bool PromptUserForFilePath(FString& OutFilePath, const bool bInImport);

	/** The display name text to show for menu entries */
	UE_API virtual FText GetDisplayName() const;

	/** The display tooltip text to show for menu entries */
	UE_API virtual FText GetDisplayTooltip() const;

	/** @return True if the export operation should prompt for a file location */
	UE_API virtual bool IsFileExport() const;

	/** @return True if this serializer supports export of ease curve presets */
	UE_API virtual bool SupportsExport() const;

	/**
	 * Exports an Ease Curve Library asset to an external file.
	 * Override to implement custom export logic (for a specific file type for example)
	 *
	 * @param InFilePath The path to the file where the library will be exported
	 * @param InWeakLibraries The Ease Curve Library assets to export
	 *
	 * @return True if the export operation was successful, false otherwise
	 */
	UE_API virtual bool Export(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries);

	/** @return True if the import operation should prompt for a file location */
	UE_API virtual bool IsFileImport() const;

	/** @return True if this serializer supports import of ease curve presets */
	UE_API virtual bool SupportsImport() const;

	/**
	 * Imports an Ease Curve Library asset from an external file.
	 * Override to implement custom import logic (for a specific file type for example)
	 *
	 * @param InFilePath The path to the file where the library will be exported
	 * @param InWeakLibraries The Ease Curve Library assets to import
	 *
	 * @return True if the import operation was successful, false otherwise
	 */
    UE_API virtual bool Import(const FString& InFilePath, TSet<TWeakObjectPtr<UEaseCurveLibrary>> InWeakLibraries);
};

#undef UE_API
